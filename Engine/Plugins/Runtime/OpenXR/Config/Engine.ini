[OpenXR]
+ProblematicOpenXRApiLayerInfos="Name=XR_APILAYER_VIVE_hand_tracking,MinVersion=2,MaxVersion=2,ExtensionAddedToFallbackWithout=XR_EXT_hand_tracking"
