{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "OpenXR",
	"Description": "OpenXR is an open VR/AR standard",
	"Category": "Virtual Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "OpenXRHMD",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "OpenXRInput",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "OpenXRAR",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "OpenXREditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "XRBase",
			"Enabled": true
		},
		{
			"Name": "EnhancedInput",
			"Enabled": true
		}
	]
}
