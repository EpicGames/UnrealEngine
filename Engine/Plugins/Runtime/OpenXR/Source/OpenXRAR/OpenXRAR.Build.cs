// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class OpenXRAR : ModuleRules
{
    public OpenXRAR(ReadOnlyTargetRules Target) : base(Target)
    {
		PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "Engine",
				"MRMesh"
			}
			);

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
				"ApplicationCore",
                "HeadMountedDisplay",
                "AugmentedReality",
                "RenderCore",
                "RHI",
				"Projects",
				"OpenXRHMD",
				"XRBase",
			}
            );
			
		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
		
        if (Target.Platform == UnrealTargetPlatform.Win64)
        {
		}

		PublicIncludePathModuleNames.Add("OpenXR");
	}
}