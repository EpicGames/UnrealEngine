// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRAR.h"
#include "IXRTrackingSystem.h"
#include "IOpenXRHMD.h"
#include "IOpenXRExtensionPlugin.h"
#include "MRMeshComponent.h"
#include "ARLifeCycleComponent.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "Editor.h"
#endif

#include <openxr/openxr.h>

#define LOCTEXT_NAMESPACE "OpenXRAR"

DECLARE_CYCLE_STAT(TEXT("Process Mesh Updates"), STAT_FOpenXRARSystem_ProcessMeshUpdates, STATGROUP_OPENXRAR);
DECLARE_CYCLE_STAT(TEXT("Process Plane Updates"), STAT_FOpenXRARSystem_ProcessPlaneUpdates, STATGROUP_OPENXRAR);
DECLARE_CYCLE_STAT(TEXT("ARTrackedGeometry Added"), STAT_FOpenXRARSystem_ARTrackedGeometryAdded_GameThread, STATGROUP_OPENXRAR);
DECLARE_CYCLE_STAT(TEXT("ARTrackedGeometry Updated"), STAT_FOpenXRARSystem_ARTrackedGeometryUpdated_GameThread, STATGROUP_OPENXRAR);
DECLARE_CYCLE_STAT(TEXT("ARTrackedGeometry Removed"), STAT_FOpenXRARSystem_ARTrackedGeometryRemoved_GameThread, STATGROUP_OPENXRAR);


FOpenXRARSystem::FOpenXRARSystem()
{
	SpawnARActorDelegateHandle = UARLifeCycleComponent::OnSpawnARActorDelegate.AddRaw(this, &FOpenXRARSystem::OnSpawnARActor);

	IModularFeatures::Get().RegisterModularFeature(IOpenXRARTrackedGeometryHolder::GetModularFeatureName(), this);
}

FOpenXRARSystem::~FOpenXRARSystem()
{
	IModularFeatures::Get().UnregisterModularFeature(IOpenXRARTrackedGeometryHolder::GetModularFeatureName(), this);

	UARLifeCycleComponent::OnSpawnARActorDelegate.Remove(SpawnARActorDelegateHandle);
}

void FOpenXRARSystem::SetTrackingSystem(IXRTrackingSystem& InTrackingSystem)
{
	TrackingSystem = &InTrackingSystem;
	check(TrackingSystem);
	OpenXRHMD = TrackingSystem->GetIOpenXRHMD();
	check(OpenXRHMD);

	for (auto Plugin : OpenXRHMD->GetExtensionPlugins())
	{
		if (CustomAnchorSupport == nullptr)
		{
			CustomAnchorSupport = Plugin->GetCustomAnchorSupport();
		}

		if (QRCapture == nullptr)
		{
			QRCapture = Plugin->GetCustomCaptureSupport(EARCaptureType::QRCode);
			if (QRCapture != nullptr)
			{
				CustomCaptureSupports.Emplace(QRCapture);
			}
		}

		if (CamCapture == nullptr)
		{
			CamCapture = Plugin->GetCustomCaptureSupport(EARCaptureType::Camera);
			if (CamCapture != nullptr)
			{
				CustomCaptureSupports.Emplace(CamCapture);
			}
		}

		if (SpatialMappingCapture == nullptr)
		{
			SpatialMappingCapture = Plugin->GetCustomCaptureSupport(EARCaptureType::SpatialMapping);
			if (SpatialMappingCapture != nullptr)
			{
				CustomCaptureSupports.Emplace(SpatialMappingCapture);
			}
		}

		if (SceneUnderstandingCapture == nullptr)
		{
			SceneUnderstandingCapture = Plugin->GetCustomCaptureSupport(EARCaptureType::SceneUnderstanding);
			if (SceneUnderstandingCapture != nullptr)
			{
				CustomCaptureSupports.Emplace(SceneUnderstandingCapture);
			}
		}

		if (HandMeshCapture == nullptr)
		{
			HandMeshCapture = Plugin->GetCustomCaptureSupport(EARCaptureType::HandMesh);
			if (HandMeshCapture != nullptr)
			{
				CustomCaptureSupports.Emplace(HandMeshCapture);
			}
		}
	}

}


void FOpenXRARSystem::OnStartARSession(UARSessionConfig* InSessionConfig) 
{ 
	SessionConfig = InSessionConfig; 

	SessionStatus.Status = EARSessionStatus::Running;

	for (auto Plugin : OpenXRHMD->GetExtensionPlugins())
	{
		Plugin->OnStartARSession(InSessionConfig);
	}
}

void FOpenXRARSystem::OnStopARSession() 
{
	for (auto Plugin : OpenXRHMD->GetExtensionPlugins())
	{
		Plugin->OnStopARSession();
	}

	SessionStatus.Status = EARSessionStatus::NotStarted;

	SessionConfig = nullptr;

	ClearAnchors();
	ClearTrackedGeometries();
}

void FOpenXRARSystem::OnPauseARSession() 
{
	for (auto Plugin : OpenXRHMD->GetExtensionPlugins())
	{
		Plugin->OnPauseARSession();
	}
}

void FOpenXRARSystem::OnSetAlignmentTransform(const FTransform& InAlignmentTransform)
{
	const FTransform& NewAlignmentTransform = InAlignmentTransform;

	TArray<UARTrackedGeometry*> AllTrackedGeometries = OnGetAllTrackedGeometries();
	for (UARTrackedGeometry* TrackedGeometry : AllTrackedGeometries)
	{
		TrackedGeometry->UpdateAlignmentTransform(NewAlignmentTransform);
	}

	TArray<UARPin*> AllARPins = OnGetAllPins();
	for (UARPin* SomePin : AllARPins)
	{
		SomePin->UpdateAlignmentTransform(NewAlignmentTransform);
	}
}



/** @return the info about whether the session is running normally or encountered some kind of error. */
FARSessionStatus FOpenXRARSystem::OnGetARSessionStatus() const
{
	return SessionStatus;
}

/** Returns true/false based on whether AR features are available */
bool FOpenXRARSystem::IsARAvailable() const
{
	return true;
}


bool FOpenXRARSystem::OnIsTrackingTypeSupported(EARSessionType SessionType) const
{
	switch (SessionType)
	{
	case EARSessionType::World:
		return true;

	case EARSessionType::Orientation:
	case EARSessionType::Face:
	case EARSessionType::Image:
	case EARSessionType::ObjectScanning:
	default:
		break;
	}
	return false;
}

//=========== Pins =============================================

/** @return a TArray of all the pins that attach components to TrackedGeometries */
TArray<UARPin*> FOpenXRARSystem::OnGetAllPins() const
{
	return ObjectPtrDecay(Pins);
}

/**
 * Given a scene component find the ARPin which it is pinned by, if any.
 */
UARPin* FOpenXRARSystem::FindARPinByComponent(const USceneComponent* Component) const
{
	for (UARPin* Pin : Pins)
	{
		if (Pin->GetPinnedComponent() == Component)
		{
			return Pin;
		}
	}

	return nullptr;
}

/**
 * Pin an Unreal Component to a location in the world.
 * Optionally, associate with a TrackedGeometry to receive transform updates that effectively attach the component to the geometry.
 *
 * @return the UARPin object that is pinning the component to the world and (optionally) a TrackedGeometry
 */
UARPin* FOpenXRARSystem::OnPinComponent(USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry /*= nullptr*/, const FName DebugName/* = NAME_None*/)
{
	if (!ensureMsgf(ComponentToPin != nullptr, TEXT("Cannot pin component.")))
	{
		return nullptr;
	}

	if (UARPin* FindResult = FindARPinByComponent(ComponentToPin))
	{
		UE_LOGF(LogOpenXRAR, Warning, "Component %ls is already pinned. Unpinning it first.", *ComponentToPin->GetReadableName());
		OnRemovePin(FindResult);
	}
	TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface = TrackingSystem->GetARCompositionComponent();

	// PinToWorld * AlignedTrackingToWorld(-1) * TrackingToAlignedTracking(-1) = PinToWorld * WorldToAlignedTracking * AlignedTrackingToTracking
	// The Worlds and AlignedTracking cancel out, and we get PinToTracking
	// But we must translate this logic into Unreal's transform API
	const FTransform& TrackingToAlignedTracking = ARSupportInterface->GetAlignmentTransform();
	const FTransform PinToTrackingTransform = PinToWorldTransform.GetRelativeTransform(TrackingSystem->GetTrackingToWorldTransform()).GetRelativeTransform(TrackingToAlignedTracking);
	

	UARPin* NewPin = NewObject<UARPin>();
	NewPin->InitARPin(ARSupportInterface.ToSharedRef(), ComponentToPin, PinToTrackingTransform, TrackedGeometry, DebugName);

	// If the user did not provide a TrackedGeometry, create an anchor for this pin.
	if (TrackedGeometry == nullptr)
	{
		if (CustomAnchorSupport != nullptr)
		{
			XrSession Session = OpenXRHMD->GetSession();
			XrTime DisplayTime = 0;
			if (OpenXRHMD->IsFocused())
			{
				// If the tracking system is not focused we will try to pin with time zero.
				// The locate should fail, but we will still setup the pin so it will work when the tracking system is working.
				DisplayTime = OpenXRHMD->GetDisplayTime();
			}
			XrSpace TrackingSpace = OpenXRHMD->GetTrackingSpace();
			float WorldToMetersScale = TrackingSystem->GetWorldToMetersScale();
			if (!CustomAnchorSupport->OnPinComponent(NewPin, Session, TrackingSpace, DisplayTime, WorldToMetersScale))
			{
				UE_LOGF(LogOpenXRAR, Error, "Component %ls failed to pin.", *ComponentToPin->GetReadableName());
			}
		}
	}

	Pins.Add(NewPin);
	return NewPin;
}


/**
 * Given a pin, remove it and stop updating the associated component based on the tracked geometry.
 * The component in question will continue to track with the world, but will not get updates specific to a TrackedGeometry.
 */
void FOpenXRARSystem::OnRemovePin(UARPin* PinToRemove) 
{
	if (PinToRemove == nullptr)
	{
		return;
	}

	Pins.RemoveSingleSwap(PinToRemove);

	if (CustomAnchorSupport != nullptr)
	{
		CustomAnchorSupport->OnRemovePin(PinToRemove);
	}
}

void FOpenXRARSystem::UpdateAnchors()
{
	if (SessionStatus.Status != EARSessionStatus::Running) { return; }

	if (CustomAnchorSupport != nullptr)
	{
		if (OpenXRHMD->IsFocused())
		{
			XrSession Session = OpenXRHMD->GetSession();
			XrTime DisplayTime = OpenXRHMD->GetDisplayTime();
			XrSpace TrackingSpace = OpenXRHMD->GetTrackingSpace();
			float WorldToMetersScale = TrackingSystem->GetWorldToMetersScale();
			for (UARPin* Pin : Pins)
			{
				CustomAnchorSupport->OnUpdatePin(Pin, Session, TrackingSpace, DisplayTime, WorldToMetersScale);
			}
		}
	}
}

bool FOpenXRARSystem::IsLocalPinSaveSupported() const
{
	return CustomAnchorSupport != nullptr && CustomAnchorSupport->IsLocalPinSaveSupported();
}

bool FOpenXRARSystem::ArePinsReadyToLoad()
{
	if (!IsLocalPinSaveSupported()) { return false; }

	return CustomAnchorSupport->ArePinsReadyToLoad();
}

void FOpenXRARSystem::LoadARPins(TMap<FName, UARPin*>& LoadedPins)
{
	if (!IsLocalPinSaveSupported()) { return; }

	CustomAnchorSupport->LoadARPins(OpenXRHMD->GetSession(),
		[&, this](FName Name)
		{
			check(IsInGameThread());
			for (auto Pin: Pins)
			{
				if (Pin->GetFName().ToString().ToLower() == Name.ToString().ToLower())
				{
					LoadedPins.Add(Name, Pin);
					return (UARPin*)nullptr;
				}
			}
			TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface = TrackingSystem->GetARCompositionComponent();

			UARPin* NewPin = NewObject<UARPin>();
			NewPin->InitARPin(ARSupportInterface.ToSharedRef(), nullptr, FTransform::Identity, nullptr, Name);

			Pins.Add(NewPin);
			LoadedPins.Add(Name, NewPin);
			return NewPin;
		});
}

bool FOpenXRARSystem::SaveARPin(FName InName, UARPin* InPin)
{
	if (!IsLocalPinSaveSupported()) { return false; }

	return CustomAnchorSupport->SaveARPin(OpenXRHMD->GetSession(), InName, InPin);
}

void FOpenXRARSystem::RemoveSavedARPin(FName InName)
{
	if (!IsLocalPinSaveSupported()) { return; }

	CustomAnchorSupport->RemoveSavedARPin(OpenXRHMD->GetSession(), InName);
}

void FOpenXRARSystem::RemoveAllSavedARPins()
{
	if (!IsLocalPinSaveSupported()) { return; }

	CustomAnchorSupport->RemoveAllSavedARPins(OpenXRHMD->GetSession());
}



void FOpenXRARSystem::ClearAnchors()
{
	TArray<UARPin*> TempPins;
	for (UARPin* Pin : Pins)
	{
		TempPins.Add(Pin);
	}

	for (UARPin* PinToRemove : TempPins)
	{
		OnRemovePin(PinToRemove);
	}
}

//=========== End of Pins =============================================

//=========== Tracked Geometries =============================================


TArray<FARTraceResult> FOpenXRARSystem::OnLineTraceTrackedObjects(const FVector2D ScreenCoord, EARLineTraceChannels TraceChannels)
{
	return {};
}

TArray<FARTraceResult> FOpenXRARSystem::OnLineTraceTrackedObjects(const FVector Start, const FVector End, EARLineTraceChannels TraceChannels)
{
	if (!TrackingSystem)
	{
		return {};
	}

	TArray<FARTraceResult> Results;

	for (auto CustomCapture : CustomCaptureSupports)
	{
		Results += CustomCapture->OnLineTraceTrackedObjects(TrackingSystem->GetARCompositionComponent(), Start, End, TraceChannels);
	}

	Results.Sort([](const FARTraceResult& A, const FARTraceResult& B)
	{
		return A.GetDistanceFromCamera() < B.GetDistanceFromCamera();
	});

	return Results;
}

TArray<UARTrackedGeometry*> FOpenXRARSystem::OnGetAllTrackedGeometries() const
{ 
	TArray<UARTrackedGeometry*> Geometries;
	// Gather all geometries
	for (auto GeoIt = TrackedGeometryGroups.CreateConstIterator(); GeoIt; ++GeoIt)
	{
		Geometries.Add(GeoIt.Value().TrackedGeometry);
	}
	return Geometries;
}

void FOpenXRARSystem::StartMeshUpdates()
{
	CurrentUpdateSync.Lock();
	CurrentUpdate = new FMeshUpdateSet();
	SUCurrentPlaneUpdate = new FPlaneUpdateSet();
}

FOpenXRMeshUpdate* FOpenXRARSystem::AllocateMeshUpdate(FGuid InGuidMeshUpdate)
{
	FOpenXRMeshUpdate* MeshUpdate = new FOpenXRMeshUpdate();
	MeshUpdate->Id = InGuidMeshUpdate;

	CurrentUpdate->GuidToMeshUpdateList.Add(MeshUpdate->Id, MeshUpdate);
	return MeshUpdate;
}

void FOpenXRARSystem::RemoveMesh(FGuid InGuidMeshUpdate)
{
	auto GTTask = FSimpleDelegateGraphTask::FDelegate::CreateThreadSafeSP(this, &FOpenXRARSystem::RemoveMesh_GameThread, InGuidMeshUpdate);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(GTTask, GET_STATID(STAT_FOpenXRARSystem_ProcessMeshUpdates), nullptr, ENamedThreads::GameThread);
}

FOpenXRPlaneUpdate* FOpenXRARSystem::AllocatePlaneUpdate(FGuid InGuidPlaneUpdate)
{
	FOpenXRPlaneUpdate* PlaneUpdate = new FOpenXRPlaneUpdate();
	PlaneUpdate->Id = InGuidPlaneUpdate;

	SUCurrentPlaneUpdate->GuidToPlaneUpdateList.Add(PlaneUpdate->Id, PlaneUpdate);
	return PlaneUpdate;
}

void FOpenXRARSystem::RemovePlane(FGuid InGuidPlaneUpdate)
{
	//RemoveMesh_GameThread is capable to remove all tracked geometries like meshes and planes
	auto GTTask = FSimpleDelegateGraphTask::FDelegate::CreateThreadSafeSP(this, &FOpenXRARSystem::RemoveMesh_GameThread, InGuidPlaneUpdate);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(GTTask, GET_STATID(STAT_FOpenXRARSystem_ProcessPlaneUpdates), nullptr, ENamedThreads::GameThread);
}

void FOpenXRARSystem::EndMeshUpdates()
{
	bool bNeedsThreadQueueing = true;
	// Lock the list to process, append our new work, and then release our work set
	{
		FScopeLock sl(&MeshUpdateListSync);
		MeshUpdateList.Add(CurrentUpdate);
		bNeedsThreadQueueing = MeshUpdateList.Num() == 1;
	}
	CurrentUpdate = nullptr;

	// Since the game thread worker works through the queue we only need queue if there is only 1 item
	if (bNeedsThreadQueueing)
	{
		// Queue a game thread processing update
		auto MeshProcessTask = FSimpleDelegateGraphTask::FDelegate::CreateThreadSafeSP(this, &FOpenXRARSystem::ProcessMeshUpdates_GameThread);
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(MeshProcessTask, GET_STATID(STAT_FOpenXRARSystem_ProcessMeshUpdates), nullptr, ENamedThreads::GameThread);
	}

	bNeedsThreadQueueing = true;
	{
		FScopeLock sl(&SUPlaneUpdateListSync);
		SUPlaneUpdateList.Add(SUCurrentPlaneUpdate);
		bNeedsThreadQueueing = SUPlaneUpdateList.Num() == 1;
	}
	SUCurrentPlaneUpdate = nullptr;

	if (bNeedsThreadQueueing)
	{
		// Queue a game thread processing update
		auto MeshProcessTask = FSimpleDelegateGraphTask::FDelegate::CreateThreadSafeSP(this, &FOpenXRARSystem::ProcessSUPlaneUpdates_GameThread);
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(MeshProcessTask, GET_STATID(STAT_FOpenXRARSystem_ProcessPlaneUpdates), nullptr, ENamedThreads::GameThread);
	}

	CurrentUpdateSync.Unlock();
}

void FOpenXRARSystem::ObjectUpdated(FOpenXRARTrackedGeometryData* InUpdate)
{
	// Convert input pointer to a SharedPtr so it will be refcounted in OnObjectUpdated_GameThread.
	TSharedPtr<FOpenXRARTrackedGeometryData> SharedUpdate = MakeShareable(InUpdate);
	ObjectUpdated(SharedUpdate);
}

void FOpenXRARSystem::ObjectUpdated(TSharedPtr<FOpenXRARTrackedGeometryData> InUpdate)
{
	auto Task = FSimpleDelegateGraphTask::FDelegate::CreateThreadSafeSP(this, &FOpenXRARSystem::OnObjectUpdated_GameThread, InUpdate);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(Task, GET_STATID(STAT_FOpenXRARSystem_ProcessPlaneUpdates), nullptr, ENamedThreads::GameThread);
}

void FOpenXRARSystem::RemoveMesh_GameThread(FGuid InGuidMeshUpdate)
{
	FTrackedGeometryGroup* FoundTrackedGeometryGroup = TrackedGeometryGroups.Find(InGuidMeshUpdate);
	if (FoundTrackedGeometryGroup != nullptr)
	{
		UARTrackedGeometry* TrackedGeometry = FoundTrackedGeometryGroup->TrackedGeometry;
		UARComponent* ARComponent = FoundTrackedGeometryGroup->ARComponent;
		AARActor* ARActor = FoundTrackedGeometryGroup->ARActor;

		check(TrackedGeometry != nullptr);

		//send the notification before we delete anything
		if (ARComponent)
		{
			ARComponent->Remove(TrackedGeometry);
			AARActor::RequestDestroyARActor(ARActor);
		}
		TrackedGeometry->SetTrackingState(EARTrackingState::NotTracking);

		// Detach the mesh component from our scene if it's valid
		UMRMeshComponent* MRMesh = TrackedGeometry->GetUnderlyingMesh();
		if (MRMesh != nullptr)
		{
			MRMesh->UnregisterComponent();
			TrackedGeometry->SetUnderlyingMesh(nullptr);
		}

		TrackedGeometryGroups.Remove(InGuidMeshUpdate);
		TriggerOnTrackableRemovedDelegates(TrackedGeometry);
	}
}

void FOpenXRARSystem::ProcessMeshUpdates_GameThread()
{
	FMeshUpdateSet* UpdateToProcess = nullptr;
	bool bIsDone = false;
	while (!bIsDone)
	{
		// Lock our game thread queue to pull the next set of updates
		{
			FScopeLock sl(&MeshUpdateListSync);
			if (MeshUpdateList.Num() > 0)
			{
				UpdateToProcess = MeshUpdateList[0];
				MeshUpdateList.RemoveAt(0);
			}
			else
			{
				bIsDone = true;
			}
		}
		// It's possible that a previous call handled the updates since we loop
		if (UpdateToProcess != nullptr)
		{
			// Iterate through the list of updates processing them
			for (TMap<FGuid, FOpenXRMeshUpdate*>::TConstIterator Iter(UpdateToProcess->GuidToMeshUpdateList); Iter; ++Iter)
			{
				FOpenXRMeshUpdate* CurrentMeshUpdate = Iter.Value();
				AddOrUpdateMesh_GameThread(CurrentMeshUpdate);
				delete CurrentMeshUpdate;
			}

			// This update is done, so delete it
			delete UpdateToProcess;
			UpdateToProcess = nullptr;
		}
	}
}

void FOpenXRARSystem::AddOrUpdateMesh_GameThread(FOpenXRMeshUpdate* CurrentMesh)
{
	bool bIsAdd = false;

	FTrackedGeometryGroup* FoundTrackedGeometryGroup = TrackedGeometryGroups.Find(CurrentMesh->Id);
	if (FoundTrackedGeometryGroup == nullptr)
	{
		// We haven't seen this one before so add it to our set
		UARTrackedGeometry* NewARTrackedGeometery = CurrentMesh->ConstructNewTrackedGeometry(nullptr);
		FTrackedGeometryGroup TrackedGeometryGroup(NewARTrackedGeometery);
		TrackedGeometryGroups.Add(CurrentMesh->Id, TrackedGeometryGroup);

		FoundTrackedGeometryGroup = TrackedGeometryGroups.Find(CurrentMesh->Id);
		check(FoundTrackedGeometryGroup);

		bIsAdd = true;
	}

	UARTrackedGeometry* NewUpdatedGeometry = FoundTrackedGeometryGroup->TrackedGeometry;
	// If the input mesh type does not support collisions, do not use with the physics system.
	if (!CurrentMesh->HasSpatialMeshUsageFlag(EARSpatialMeshUsageFlags::Collision)
		&& NewUpdatedGeometry != nullptr
		&& NewUpdatedGeometry->GetUnderlyingMesh() != nullptr)
	{
		NewUpdatedGeometry->GetUnderlyingMesh()->SetNeverCreateCollisionMesh(true);
	}

	// Update the tracked geometry before calling the add or update delegate so the event has valid data.
	CurrentMesh->UpdateTrackedGeometry(NewUpdatedGeometry, TrackingSystem->GetARCompositionComponent());

	// Trigger the proper notification delegate
	if (bIsAdd)
	{
		if (SessionConfig != nullptr)
		{
			AARActor::RequestSpawnARActor(CurrentMesh->Id, SessionConfig->GetMeshComponentClass());
		}
	} 
	else
	{
		UARComponent* NewUpdatedARComponent = FoundTrackedGeometryGroup->ARComponent;
		if (NewUpdatedARComponent)
		{
			NewUpdatedARComponent->Update(NewUpdatedGeometry);
			TriggerOnTrackableUpdatedDelegates(NewUpdatedGeometry);
		}
	}
}

void FOpenXRARSystem::ProcessSUPlaneUpdates_GameThread()
{
	FPlaneUpdateSet* UpdateToProcess = nullptr;
	bool bIsDone = false;
	while (!bIsDone)
	{
		// Lock our game thread queue to pull the next set of updates
		{
			FScopeLock sl(&SUPlaneUpdateListSync);
			if (SUPlaneUpdateList.Num() > 0)
			{
				UpdateToProcess = SUPlaneUpdateList[0];
				SUPlaneUpdateList.RemoveAt(0);
			}
			else
			{
				bIsDone = true;
			}
		}
		// It's possible that a previous call handled the updates since we loop
		if (UpdateToProcess != nullptr)
		{
			// Iterate through the list of updates processing them
			for (TMap<FGuid, FOpenXRPlaneUpdate*>::TConstIterator Iter(UpdateToProcess->GuidToPlaneUpdateList); Iter; ++Iter)
			{
				FOpenXRPlaneUpdate* CurrentPlaneUpdate = Iter.Value();
				AddOrUpdatePlane_GameThread(CurrentPlaneUpdate);
				delete CurrentPlaneUpdate;
			}

			// This update is done, so delete it
			delete UpdateToProcess;
			UpdateToProcess = nullptr;
		}
	}
}

void FOpenXRARSystem::AddOrUpdatePlane_GameThread(FOpenXRPlaneUpdate* CurrentPlaneUpdate)
{
	bool bIsAdd = false;

	FTrackedGeometryGroup* FoundTrackedGeometryGroup = TrackedGeometryGroups.Find(CurrentPlaneUpdate->Id);
	if (FoundTrackedGeometryGroup == nullptr)
	{
		// We haven't seen this one before so add it to our set
		UARTrackedGeometry* NewARTrackedGeometery = CurrentPlaneUpdate->ConstructNewTrackedGeometry(nullptr);
		FTrackedGeometryGroup TrackedGeometryGroup(NewARTrackedGeometery);
		TrackedGeometryGroups.Add(CurrentPlaneUpdate->Id, TrackedGeometryGroup);

		FoundTrackedGeometryGroup = TrackedGeometryGroups.Find(CurrentPlaneUpdate->Id);
		check(FoundTrackedGeometryGroup);

		bIsAdd = true;
	}

	UARTrackedGeometry* NewUpdatedGeometry = FoundTrackedGeometryGroup->TrackedGeometry;
	// If the input mesh type does not support collisions, do not use with the physics system.
	if (!CurrentPlaneUpdate->HasSpatialMeshUsageFlag(EARSpatialMeshUsageFlags::Collision)
		&& NewUpdatedGeometry != nullptr
		&& NewUpdatedGeometry->GetUnderlyingMesh() != nullptr)
	{
		NewUpdatedGeometry->GetUnderlyingMesh()->SetNeverCreateCollisionMesh(true);
	}

	CurrentPlaneUpdate->UpdateTrackedGeometry(NewUpdatedGeometry, TrackingSystem->GetARCompositionComponent());

	// Trigger the proper notification delegate
	if (bIsAdd)
	{
		if (SessionConfig != nullptr)
		{
			AARActor::RequestSpawnARActor(CurrentPlaneUpdate->Id, SessionConfig->GetPlaneComponentClass());
		}
	}
	else
	{
		UARComponent* NewUpdatedARComponent = FoundTrackedGeometryGroup->ARComponent;
		if (NewUpdatedARComponent)
		{
			NewUpdatedARComponent->Update(NewUpdatedGeometry);
			TriggerOnTrackableUpdatedDelegates(NewUpdatedGeometry);
		}
	}
}

void FOpenXRARSystem::OnObjectUpdated_GameThread(TSharedPtr<FOpenXRARTrackedGeometryData> InUpdate)
{
	if (InUpdate == nullptr)
	{
		return;
	}
	
	FTrackedGeometryGroup* FoundTrackedGeometryGroup = TrackedGeometryGroups.Find(InUpdate->Id);
	if (FoundTrackedGeometryGroup == nullptr)
	{
		return;
	}

	UARTrackedGeometry* NewUpdatedGeometry = FoundTrackedGeometryGroup->TrackedGeometry;
	UARComponent* NewUpdatedARComponent = FoundTrackedGeometryGroup->ARComponent;

	if (NewUpdatedGeometry == nullptr)
	{
		return;
	}

	// Update the tracking data, it MUST be done after UpdateMesh
	NewUpdatedGeometry->UpdateTrackedGeometry(TrackingSystem->GetARCompositionComponent().ToSharedRef(),
		GFrameCounter,
		FPlatformTime::Seconds(),
		InUpdate->LocalToTrackingTransform,
		TrackingSystem->GetARCompositionComponent()->GetAlignmentTransform());
	// Mark this as a world mesh that isn't recognized as a particular scene type, since it is loose triangles
	NewUpdatedGeometry->SetTrackingState(InUpdate->TrackingState);

	// Trigger the proper notification delegate
	if (NewUpdatedARComponent)
	{
		NewUpdatedARComponent->Update(NewUpdatedGeometry);
		TriggerOnTrackableUpdatedDelegates(NewUpdatedGeometry);
	}
}

void FOpenXRARSystem::ClearTrackedGeometries()
{
	for (auto GeoIt = TrackedGeometryGroups.CreateIterator(); GeoIt; ++GeoIt)
	{
		FTrackedGeometryGroup& TrackedGeometryGroup = GeoIt.Value();
		if (TrackedGeometryGroup.ARActor)
		{
			AARActor::RequestDestroyARActor(TrackedGeometryGroup.ARActor);
		}
		// Remove the occlusion mesh if present
		UARTrackedGeometry* TrackedGeometryBeingRemoved = TrackedGeometryGroup.TrackedGeometry;
		check(TrackedGeometryBeingRemoved);
		UMRMeshComponent* MRMesh = TrackedGeometryBeingRemoved->GetUnderlyingMesh();
		if (MRMesh != nullptr)
		{
			MRMesh->DestroyComponent();
			TrackedGeometryBeingRemoved->SetUnderlyingMesh(nullptr);
		}
	}
	TrackedGeometryGroups.Empty();
}


void FOpenXRARSystem::OnSpawnARActor(AARActor* NewARActor, UARComponent* NewARComponent, FGuid NativeID)
{
	FTrackedGeometryGroup* TrackedGeometryGroup = TrackedGeometryGroups.Find(NativeID);
	if (TrackedGeometryGroup != nullptr)
	{
		//this should still be null
		check(TrackedGeometryGroup->ARActor == nullptr);
		check(TrackedGeometryGroup->ARComponent == nullptr);

		check(NewARActor);
		check(NewARComponent);

		TrackedGeometryGroup->ARActor = NewARActor;
		TrackedGeometryGroup->ARComponent = NewARComponent;

		//NOW, we can make the callbacks
		TrackedGeometryGroup->ARComponent->Update(TrackedGeometryGroup->TrackedGeometry);
		TriggerOnTrackableAddedDelegates(TrackedGeometryGroup->TrackedGeometry);
	}
	else
	{
		UE_LOGF(LogOpenXRAR, Warning, "AR NativeID not found.  Make sure to set this on the ARComponent!");
	}
}

void FOpenXRARSystem::ARTrackedGeometryAdded(FOpenXRARTrackedGeometryData* InData)
{
	TSharedPtr<FOpenXRARTrackedGeometryData> SharedData = MakeShareable(InData);
	ARTrackedGeometryAdded(SharedData);
}

void FOpenXRARSystem::ARTrackedGeometryUpdated(FOpenXRARTrackedGeometryData* InData)
{
	TSharedPtr<FOpenXRARTrackedGeometryData> SharedData = MakeShareable(InData);
	ARTrackedGeometryUpdated(SharedData);
}

void FOpenXRARSystem::ARTrackedGeometryRemoved(FOpenXRARTrackedGeometryData* InData)
{
	TSharedPtr<FOpenXRARTrackedGeometryData> SharedData = MakeShareable(InData);
	ARTrackedGeometryRemoved(SharedData);
}

void FOpenXRARSystem::ARTrackedGeometryAdded(TSharedPtr<FOpenXRARTrackedGeometryData> InData)
{
	auto Task = FSimpleDelegateGraphTask::FDelegate::CreateThreadSafeSP(this, &FOpenXRARSystem::ARTrackedGeometryAdded_GameThread, InData);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(Task, GET_STATID(STAT_FOpenXRARSystem_ARTrackedGeometryAdded_GameThread), nullptr, ENamedThreads::GameThread);
}

void FOpenXRARSystem::ARTrackedGeometryUpdated(TSharedPtr<FOpenXRARTrackedGeometryData> InData)
{
	auto Task = FSimpleDelegateGraphTask::FDelegate::CreateThreadSafeSP(this, &FOpenXRARSystem::ARTrackedGeometryUpdated_GameThread, InData);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(Task, GET_STATID(STAT_FOpenXRARSystem_ARTrackedGeometryUpdated_GameThread), nullptr, ENamedThreads::GameThread);

}

void FOpenXRARSystem::ARTrackedGeometryRemoved(TSharedPtr<FOpenXRARTrackedGeometryData> InData)
{
	auto Task = FSimpleDelegateGraphTask::FDelegate::CreateThreadSafeSP(this, &FOpenXRARSystem::ARTrackedGeometryRemoved_GameThread, InData);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(Task, GET_STATID(STAT_FOpenXRARSystem_ARTrackedGeometryRemoved_GameThread), nullptr, ENamedThreads::GameThread);
}


void FOpenXRARSystem::ARTrackedGeometryAdded_GameThread(TSharedPtr<FOpenXRARTrackedGeometryData> InData)
{
	if (InData == nullptr) 
	{ 
		return; 
	}

	// We haven't seen this one before so add it to our set
	FTrackedGeometryGroup TrackedGeometryGroup(InData->ConstructNewTrackedGeometry(TrackingSystem->GetARCompositionComponent()));
	TrackedGeometryGroups.Add(InData->Id, TrackedGeometryGroup);

	InData->UpdateTrackedGeometry(TrackedGeometryGroup.TrackedGeometry, TrackingSystem->GetARCompositionComponent());

	if (SessionConfig != nullptr)
	{
		AARActor::RequestSpawnARActor(InData->Id, SessionConfig->GetQRCodeComponentClass());
	}
}

void FOpenXRARSystem::ARTrackedGeometryUpdated_GameThread(TSharedPtr<FOpenXRARTrackedGeometryData> InData)
{
	if (InData == nullptr) 
	{ 
		return; 
	}

	FTrackedGeometryGroup* TrackedGeometryGroup = TrackedGeometryGroups.Find(InData->Id);
	if (TrackedGeometryGroup != nullptr)
	{
		UARTrackedGeometry* FoundGeometry = TrackedGeometryGroup->TrackedGeometry;
		UARTrackedQRCode* UpdatedQRCode = Cast<UARTrackedQRCode>(FoundGeometry);
		check(UpdatedQRCode != nullptr);

		InData->UpdateTrackedGeometry(UpdatedQRCode, TrackingSystem->GetARCompositionComponent());

		if (TrackedGeometryGroup->ARComponent != nullptr)
		{
			TrackedGeometryGroup->ARComponent->Update(UpdatedQRCode);
			TriggerOnTrackableUpdatedDelegates(UpdatedQRCode);
		}
	}
}

void FOpenXRARSystem::ARTrackedGeometryRemoved_GameThread(TSharedPtr<FOpenXRARTrackedGeometryData> InData)
{
	if (InData == nullptr) 
	{ 
		return; 
	}

	FTrackedGeometryGroup* TrackedGeometryGroup = TrackedGeometryGroups.Find(InData->Id);
	if (TrackedGeometryGroup != nullptr)
	{
		if (TrackedGeometryGroup->ARComponent)
		{
			check(TrackedGeometryGroup->ARActor);

			TrackedGeometryGroup->ARComponent->Remove(TrackedGeometryGroup->TrackedGeometry);
			AARActor::RequestDestroyARActor(TrackedGeometryGroup->ARActor);
		}

		check(TrackedGeometryGroup->TrackedGeometry);
		TrackedGeometryGroup->TrackedGeometry->SetTrackingState(EARTrackingState::NotTracking);

		TrackedGeometryGroups.Remove(InData->Id);
		TriggerOnTrackableRemovedDelegates(TrackedGeometryGroup->TrackedGeometry);
	}
}



//=========== End of Tracked Geometries =============================================

//=========== AR Capture =============================================

UARTexture* FOpenXRARSystem::OnGetARTexture(EARTextureType TextureType) const 
{
	if (CamCapture)
	{
		return CamCapture->OnGetARTexture(TextureType);
	}
	return nullptr; 
}

bool FOpenXRARSystem::OnToggleARCapture(const bool bOnOff, const EARCaptureType CaptureType) 
{
	switch (CaptureType)
	{
	case EARCaptureType::Camera:
		if (CamCapture)
		{
			return CamCapture->OnToggleARCapture(bOnOff);
		}
		break;
	case EARCaptureType::QRCode:
		if (QRCapture)
		{
			return QRCapture->OnToggleARCapture(bOnOff);
		}
		break;
	case EARCaptureType::SpatialMapping:
		if (SpatialMappingCapture)
		{
			return SpatialMappingCapture->OnToggleARCapture(bOnOff);
		}
		break;
	case EARCaptureType::SceneUnderstanding:
		if (SceneUnderstandingCapture)
		{
			return SceneUnderstandingCapture->OnToggleARCapture(bOnOff);
		}
		break;
	case EARCaptureType::HandMesh:
		if (HandMeshCapture)
		{
			return HandMeshCapture->OnToggleARCapture(bOnOff);
		}
		break;
	}

	return false;
}

bool FOpenXRARSystem::OnGetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const
{
	if (CamCapture)
	{
		return CamCapture->OnGetCameraIntrinsics(OutCameraIntrinsics);
	}
	return false;
}

//=========== End of AR Capture =============================================


bool FOpenXRARSystem::OnStartARGameFrame(FWorldContext& WorldContext)
{
	UpdateAnchors();

	return true;
}

/**
* Pure virtual that must be overloaded by the inheriting class. Use this
* method to serialize any UObjects contained that you wish to keep around.
*
* @param Collector The collector of referenced objects.
*/
void FOpenXRARSystem::AddReferencedObjects(FReferenceCollector& Collector) 
{
	Collector.AddReferencedObject(SessionConfig);
	Collector.AddReferencedObjects(Pins);

	// Iterate all geometries
	for (auto GeoIt = TrackedGeometryGroups.CreateIterator(); GeoIt; ++GeoIt)
	{
		FTrackedGeometryGroup& TrackedGeometryGroup = GeoIt.Value();

		Collector.AddReferencedObject(TrackedGeometryGroup.TrackedGeometry);
		Collector.AddReferencedObject(TrackedGeometryGroup.ARActor);
		Collector.AddReferencedObject(TrackedGeometryGroup.ARComponent);
	}
}



/** Used to init our AR system */
IARSystemSupport* OpenXRARModuleImpl::CreateARSystem()
{
	IARSystemSupport* ARSystemPtr = nullptr;
	if (!ARSystem.IsValid())
	{
		ARSystem = MakeShareable(new FOpenXRARSystem());
	}
	ARSystemPtr = ARSystem.Get();
	return ARSystemPtr;
}

void OpenXRARModuleImpl::SetTrackingSystem(IXRTrackingSystem& InTrackingSystem)
{
	ARSystem->SetTrackingSystem(InTrackingSystem);
}

void OpenXRARModuleImpl::StartupModule()
{
	ensureMsgf(FModuleManager::Get().LoadModule("AugmentedReality"), TEXT("OpenXR depends on the AugmentedReality module."));
}

void OpenXRARModuleImpl::ShutdownModule()
{
	ARSystem = nullptr;
}

bool OpenXRARModuleImpl::GetExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
	return true;
}

IOpenXRARTrackedMeshHolder* OpenXRARModuleImpl::GetTrackedMeshHolder()
{
	return ARSystem.Get();
}



IMPLEMENT_MODULE(OpenXRARModuleImpl, OpenXRAR)

DEFINE_LOG_CATEGORY(LogOpenXRAR)

#undef LOCTEXT_NAMESPACE
