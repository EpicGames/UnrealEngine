// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOpenXRARTrackedGeometryHolder.h"
#include "IOpenXRHMDModule.h"
#include "ARSystemSupportBase.h"
#include "ARTraceResult.h"
#include "ARPin.h"
#include "OpenXRCore.h"
#include "ARActor.h"
#include "IOpenXRARModule.h"

class IOpenXRHMD;
class IXRTrackingSystem;

DECLARE_STATS_GROUP(TEXT("OpenXRAR"), STATGROUP_OPENXRAR, STATCAT_Advanced);



class FOpenXRARSystem :
	public FARSystemSupportBase,
	public IOpenXRARTrackedMeshHolder,
	public IOpenXRARTrackedGeometryHolder,
	public FGCObject,
	public TSharedFromThis<FOpenXRARSystem, ESPMode::ThreadSafe>
{
public:
	/** A set of updates to be processed at once */
	struct FMeshUpdateSet
	{
		FMeshUpdateSet()
		{
		}
		~FMeshUpdateSet()
		{
			GuidToMeshUpdateList.Empty();
		}

		TMap<FGuid, FOpenXRMeshUpdate*> GuidToMeshUpdateList;
	};

	struct FPlaneUpdateSet
	{
		FPlaneUpdateSet()
		{
		}
		~FPlaneUpdateSet()
		{
			GuidToPlaneUpdateList.Empty();
		}

		TMap<FGuid, FOpenXRPlaneUpdate*> GuidToPlaneUpdateList;
	};

	FOpenXRARSystem();
	virtual ~FOpenXRARSystem();

	void SetTrackingSystem(IXRTrackingSystem& InTrackingSystem);

	/** Invoked after the base AR system has been initialized. */
	virtual void OnARSystemInitialized() {}

	virtual bool OnStartARGameFrame(FWorldContext& WorldContext);

	/** @return the tracking quality; if unable to determine tracking quality, return EARTrackingQuality::NotAvailable */
	virtual EARTrackingQuality OnGetTrackingQuality() const { return EARTrackingQuality::NotTracking; };

	/** @return the reason of limited tracking quality; if the state is not limited, return EARTrackingQualityReason::None */
	virtual EARTrackingQualityReason OnGetTrackingQualityReason() const { return EARTrackingQualityReason::None; }

	/**
	 * Start the AR system.
	 *
	 * @param SessionType The type of AR session to create
	 *
	 * @return true if the system was successfully started
	 */
	virtual void OnStartARSession(UARSessionConfig* SessionConfig);

	/** Stop the AR system but leave its internal state intact. */
	virtual void OnPauseARSession();

	/** Stop the AR system and reset its internal state; this task must succeed. */
	virtual void OnStopARSession();

	/** @return the info about whether the session is running normally or encountered some kind of error. */
	virtual FARSessionStatus OnGetARSessionStatus() const;

	/** Returns true/false based on whether AR features are available */
	virtual bool IsARAvailable() const;

	/**
	 * Set a transform that will align the Tracking Space origin to the World Space origin.
	 * This is useful for supporting static geometry and static lighting in AR.
	 * Note: Usually, an app will ask the user to select an appropriate location for some
	 * experience. This allows us to choose an appropriate alignment transform.
	 */
	virtual void OnSetAlignmentTransform(const FTransform& InAlignmentTransform);

	/**
	 * Trace all the tracked geometries and determine which have been hit by a ray cast from `ScreenCoord`.
	 * Only geometries specified in `TraceChannels` are considered.
	 *
	 * @return a list of all the geometries that were hit, sorted by distance
	 */
	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects(const FVector2D ScreenCoord, EARLineTraceChannels TraceChannels);

	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects(const FVector Start, const FVector End, EARLineTraceChannels TraceChannels);

	/** @return a TArray of all the tracked geometries known to your ar system */
	virtual TArray<UARTrackedGeometry*> OnGetAllTrackedGeometries() const;

	/** @return a TArray of all the pins that attach components to TrackedGeometries */
	virtual TArray<UARPin*> OnGetAllPins() const;

	/** @return whether the specified tracking type is supported by this device */
	virtual bool OnIsTrackingTypeSupported(EARSessionType SessionType) const;

	/** @return the best available light estimate; nullptr if light estimation is inactive or not available */
	virtual UARLightEstimate* OnGetCurrentLightEstimate() const { return nullptr; }

	/**
	 * Given a scene component find the ARPin which it is pinned by, if any.
	 */
	virtual UARPin* FindARPinByComponent(const USceneComponent* Component) const;
	/**
	 * Pin an Unreal Component to a location in the world.
	 * Optionally, associate with a TrackedGeometry to receive transform updates that effectively attach the component to the geometry.
	 *
	 * @return the UARPin object that is pinning the component to the world and (optionally) a TrackedGeometry
	 */
	virtual UARPin* OnPinComponent(USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry = nullptr, const FName DebugName = NAME_None);

	/**
	 * Given a pin, remove it and stop updating the associated component based on the tracked geometry.
	 * The component in question will continue to track with the world, but will not get updates specific to a TrackedGeometry.
	 */
	virtual void OnRemovePin(UARPin* PinToRemove);

	/** Tells the ARSystem to generate a capture probe at the specified location if supported */
	virtual bool OnAddManualEnvironmentCaptureProbe(FVector Location, FVector Extent) { return false; }

	/** Generates a UARCandidateObject from the point cloud data within the location and its extent using an async task */
	virtual TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> OnGetCandidateObject(FVector Location, FVector Extent) const { return {}; }

	/** Saves the AR world to a byte array using an async task */
	virtual TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> OnSaveWorld() const { return {}; }

	/** @return the current mapping status */
	virtual EARWorldMappingState OnGetWorldMappingStatus() const { return EARWorldMappingState::NotAvailable; }

	/** @return The list of supported video formats for this device and session type */
	virtual TArray<FARVideoFormat> OnGetSupportedVideoFormats(EARSessionType SessionType) const { return {}; }

	/** @return the current point cloud data for the ar scene */
	virtual TArray<FVector> OnGetPointCloud() const { return {}; }

	/** Add candidate image at runtime @return True if it added the iamge successfully */
	virtual bool OnAddRuntimeCandidateImage(UARSessionConfig* InSessionConfig, UTexture2D* CandidateTexture, FString FriendlyName, float PhysicalWidth) { return false; }

	virtual void* GetARSessionRawPointer() { return nullptr; }
	virtual void* GetGameThreadARFrameRawPointer() { return nullptr; }

	/** @return if a particular session feature is supported on this device */
	virtual bool OnIsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature) const { return false; }

	/** @return all the tracked 2D poses in AR */
	virtual TArray<FARPose2D> OnGetTracked2DPose() const { return {}; }

	/** @return the required scene construction method is supported */
	virtual bool OnIsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod) const { return false; }

	virtual bool OnAddTrackedPointWithName(const FTransform& WorldTransform, const FString& PointName, bool bDeletePointsWithSameName) { return false; }

	/** @return the max number of faces can be tracked at the same time */
	virtual int32 OnGetNumberOfTrackedFacesSupported() const { return 1; }

	/** @return the AR texture for the specified type */
	virtual UARTexture* OnGetARTexture(EARTextureType TextureType) const;

	virtual bool OnToggleARCapture(const bool bOnOff, const EARCaptureType CaptureType) override;
	virtual bool OnGetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const override;

	// ARPin Local Store support.
	// Some Platforms/Devices have the ability to persist AR Anchors (real world positiosn) to the device or user account.
	// They are saved and loaded with a string identifier.

	virtual bool IsLocalPinSaveSupported() const;

	virtual bool ArePinsReadyToLoad();

	virtual void LoadARPins(TMap<FName, UARPin*>& LoadedPins);

	virtual bool SaveARPin(FName InName, UARPin* InPin);

	virtual void RemoveSavedARPin(FName InName);

	virtual void RemoveAllSavedARPins();

	/**
 * Pure virtual that must be overloaded by the inheriting class. Use this
 * method to serialize any UObjects contained that you wish to keep around.
 *
 * @param Collector The collector of referenced objects.
 */
	virtual void AddReferencedObjects(FReferenceCollector& Collector);
	virtual FString GetReferencerName() const override
	{
		return TEXT("FOpenXRARSystem");
	}

private:
	IXRTrackingSystem* TrackingSystem = nullptr;
	IOpenXRHMD* OpenXRHMD = nullptr;

	class IOpenXRCustomAnchorSupport* CustomAnchorSupport = nullptr;

	FARSessionStatus SessionStatus;

	void UpdateAnchors();
	void ClearAnchors();

	//
	// PROPERTIES REPORTED TO FGCObject
	// ...
	TObjectPtr<UARSessionConfig> SessionConfig = nullptr;
	TArray<TObjectPtr<UARPin>> Pins;
	TMap<FGuid, FTrackedGeometryGroup> TrackedGeometryGroups;
	// ...
	// PROPERTIES REPORTED TO FGCObject
	//


	//IOpenXRARTrackedMeshHolder
	virtual void StartMeshUpdates();
	virtual FOpenXRMeshUpdate* AllocateMeshUpdate(FGuid InGuidMeshUpdate);
	virtual void RemoveMesh(FGuid InGuidMeshUpdate);
	virtual FOpenXRPlaneUpdate* AllocatePlaneUpdate(FGuid InGuidPlaneUpdate);
	virtual void RemovePlane(FGuid InGuidPlaneUpdate);
	virtual void EndMeshUpdates();
	virtual void ObjectUpdated(FOpenXRARTrackedGeometryData* InUpdate);
	virtual void ObjectUpdated(TSharedPtr<FOpenXRARTrackedGeometryData> InUpdate);

	void RemoveMesh_GameThread(FGuid InGuidMeshUpdate);
	void ProcessMeshUpdates_GameThread();
	void AddOrUpdateMesh_GameThread(FOpenXRMeshUpdate* CurrentMesh);
	void ProcessSUPlaneUpdates_GameThread();
	void AddOrUpdatePlane_GameThread(FOpenXRPlaneUpdate* CurrentPlane);
	void OnSpawnARActor(AARActor* NewARActor, UARComponent* NewARComponent, FGuid NativeID);
	void OnObjectUpdated_GameThread(TSharedPtr<FOpenXRARTrackedGeometryData> InUpdate);

	/** Removes all tracked geometries, marking them as not tracked and sending the delegate event */
	void ClearTrackedGeometries();

	/** Used to lock access to the update list that will be queued for the game thread */
	FCriticalSection CurrentUpdateSync;
	/** This pointer is locked until the list construction is complete, where this gets queued for game thread processing */
	FMeshUpdateSet* CurrentUpdate;
	/** Controls the access to the queue of mesh updates for the game thread to process */
	FCriticalSection MeshUpdateListSync;
	/** List of mesh updates for the game thread to process */
	TArray<FMeshUpdateSet*> MeshUpdateList;
	/** Holds the set of last known meshes so we can detect removed meshes. Only touched on the game thread */
	TSet<FGuid> LastKnownMeshes;
	/** This pointer is locked until the list construction is complete, where this gets queued for game thread processing */
	FPlaneUpdateSet* SUCurrentPlaneUpdate;
	/** Controls the access to the queue of plane updates for the game thread to process */
	FCriticalSection SUPlaneUpdateListSync;
	/** List of plane updates for the game thread to process */
	TArray<FPlaneUpdateSet*> SUPlaneUpdateList;

	//for networked callbacks
	FDelegateHandle SpawnARActorDelegateHandle;


	//IOpenXRARTrackedGeometryHolder
	virtual void ARTrackedGeometryAdded(FOpenXRARTrackedGeometryData* InData);
	virtual void ARTrackedGeometryUpdated(FOpenXRARTrackedGeometryData* InData);
	virtual void ARTrackedGeometryRemoved(FOpenXRARTrackedGeometryData* InData);

	virtual void ARTrackedGeometryAdded(TSharedPtr<FOpenXRARTrackedGeometryData> InData);
	virtual void ARTrackedGeometryUpdated(TSharedPtr<FOpenXRARTrackedGeometryData> InData);
	virtual void ARTrackedGeometryRemoved(TSharedPtr<FOpenXRARTrackedGeometryData> InData);

	void ARTrackedGeometryAdded_GameThread(TSharedPtr<FOpenXRARTrackedGeometryData> InData);
	void ARTrackedGeometryUpdated_GameThread(TSharedPtr<FOpenXRARTrackedGeometryData> InData);
	void ARTrackedGeometryRemoved_GameThread(TSharedPtr<FOpenXRARTrackedGeometryData> InData);

	class IOpenXRCustomCaptureSupport* QRCapture = nullptr;
	class IOpenXRCustomCaptureSupport* CamCapture = nullptr;
	class IOpenXRCustomCaptureSupport* SpatialMappingCapture = nullptr;
	class IOpenXRCustomCaptureSupport* SceneUnderstandingCapture = nullptr;
	class IOpenXRCustomCaptureSupport* HandMeshCapture = nullptr;

	TArray<IOpenXRCustomCaptureSupport*> CustomCaptureSupports;
};


class OpenXRARModuleImpl :
	public IOpenXRARModule
{
public:
	/** Used to init our AR system */
	virtual IARSystemSupport* CreateARSystem();
	virtual void SetTrackingSystem(IXRTrackingSystem& InTrackingSystem);

	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

	virtual bool GetExtensions(TArray<const ANSICHAR*>& OutExtensions) override;

	virtual class IOpenXRARTrackedMeshHolder * GetTrackedMeshHolder() override;


private:
	TSharedPtr<FOpenXRARSystem, ESPMode::ThreadSafe> ARSystem;
};

DECLARE_LOG_CATEGORY_EXTERN(LogOpenXRAR, Log, All);
