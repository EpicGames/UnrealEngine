// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOpenXRARTrackedGeometryHolder.h"
#include "MRMeshBufferDefines.h"
#include "MRMeshComponent.h"
#include "IXRTrackingSystem.h"

UARTrackedGeometry* FOpenXRQRCodeData::ConstructNewTrackedGeometry(TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface)
{
	check(IsInGameThread());

	UARTrackedQRCode* NewQRCode = NewObject<UARTrackedQRCode>();
	NewQRCode->UniqueId = Id;
	return NewQRCode;
};

void FOpenXRQRCodeData::UpdateTrackedGeometry(UARTrackedGeometry* TrackedGeometry, TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface)
{
	check(IsInGameThread());

	UARTrackedQRCode* UpdatedQRCode = Cast<UARTrackedQRCode>(TrackedGeometry);
	check(UpdatedQRCode != nullptr);

	UpdatedQRCode->UpdateTrackedGeometry(ARSupportInterface.ToSharedRef(),
		GFrameCounter,
		Timestamp,
		LocalToTrackingTransform,
		ARSupportInterface->GetAlignmentTransform(),
		Size,
		QRCode,
		Version);
	UpdatedQRCode->SetTrackingState(TrackingState);
}


UARTrackedGeometry* FOpenXRMeshUpdate::ConstructNewTrackedGeometry(TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface)
{
	check(IsInGameThread());

	UARMeshGeometry* NewMesh = NewObject<UARMeshGeometry>();
	NewMesh->UniqueId = Id;
	return NewMesh;
};

void FOpenXRMeshUpdate::UpdateTrackedGeometry(UARTrackedGeometry* TrackedGeometry, TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface)
{
	check(IsInGameThread());

	UARTrackedGeometry* UpdatedGeometry = Cast<UARTrackedGeometry>(TrackedGeometry);
	check(UpdatedGeometry != nullptr);

	if (Vertices.Num() > 0)
	{
		// Update MRMesh if it's available
		if (auto MRMesh = UpdatedGeometry->GetUnderlyingMesh())
		{
			IXRTrackingSystem* XRTrackingSystem = GEngine->XRSystem.Get();
			FTransform MeshTransform = LocalToTrackingTransform * XRTrackingSystem->GetTrackingToWorldTransform();

			// MRMesh takes ownership of the data in the arrays at this point
			MRMesh->UpdateMesh(MeshTransform.GetLocation(), MeshTransform.GetRotation(), MeshTransform.GetScale3D(), Vertices, Indices);
		}
	}

	// Update the tracking data, it MUST be done after UpdateMesh
	UpdatedGeometry->UpdateTrackedGeometry(ARSupportInterface.ToSharedRef(),
		GFrameCounter,
		FPlatformTime::Seconds(),
		LocalToTrackingTransform,
		ARSupportInterface->GetAlignmentTransform());
	// Mark this as a world mesh that isn't recognized as a particular scene type, since it is loose triangles
	UpdatedGeometry->SetObjectClassification(Type);
	UpdatedGeometry->SetSpatialMeshUsageFlags(SpatialMeshUsageFlags);
	UpdatedGeometry->SetTrackingState(TrackingState);
}

UARTrackedGeometry* FOpenXRPlaneUpdate::ConstructNewTrackedGeometry(TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface)
{
	check(IsInGameThread());

	UARPlaneGeometry* NewPlane = NewObject<UARPlaneGeometry>();
	NewPlane->UniqueId = Id;
	return NewPlane;
}

void FOpenXRPlaneUpdate::UpdateTrackedGeometry(UARTrackedGeometry* TrackedGeometry, TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface)
{
	// Add the occlusion geo if configured
	if (ARSupportInterface->GetSessionConfig().bGenerateMeshDataFromTrackedGeometry)
	{
		if (auto MRMesh = TrackedGeometry->GetUnderlyingMesh())
		{
			// Generate the mesh from the reference image's sizes
			TArray<FVector> Vertices;
			Vertices.Reset(4);
			Vertices.Add(Extent);
			Vertices.Add(FVector(Extent.X, -Extent.Y, Extent.Z));
			Vertices.Add(FVector(-Extent.X, -Extent.Y, Extent.Z));
			Vertices.Add(FVector(-Extent.X, Extent.Y, Extent.Z));

			// Two triangles
			TArray<MRMESH_INDEX_TYPE> Indices;
			Indices.Reset(6);
			Indices.Add(0);
			Indices.Add(1);
			Indices.Add(2);
			Indices.Add(2);
			Indices.Add(3);
			Indices.Add(0);

			IXRTrackingSystem* XRTrackingSystem = GEngine->XRSystem.Get();
			FTransform MeshTransform = LocalToTrackingTransform * XRTrackingSystem->GetTrackingToWorldTransform();

			// MRMesh takes ownership of the data in the arrays at this point
			MRMesh->UpdateMesh(MeshTransform.GetLocation(), MeshTransform.GetRotation(), MeshTransform.GetScale3D(), Vertices, Indices);
		}
	}

	UARPlaneGeometry* NewPlane = Cast<UARPlaneGeometry>(TrackedGeometry);

	// Update the tracking data, it MUST be done after UpdateMesh
	if (NewPlane)
	{
		NewPlane->UpdateTrackedGeometry(ARSupportInterface.ToSharedRef(),
			GFrameCounter,
			FPlatformTime::Seconds(),
			LocalToTrackingTransform,
			ARSupportInterface->GetAlignmentTransform(),
			FVector::ZeroVector,
			Extent);

	}
	else
	{
		TrackedGeometry->UpdateTrackedGeometry(ARSupportInterface.ToSharedRef(),
			GFrameCounter,
			FPlatformTime::Seconds(),
			LocalToTrackingTransform,
			ARSupportInterface->GetAlignmentTransform());

	}

	// Mark this as a world mesh that isn't recognized as a particular scene type, since it is loose triangles
	TrackedGeometry->SetObjectClassification(Type);
	TrackedGeometry->SetSpatialMeshUsageFlags(SpatialMeshUsageFlags);
	// This must be called AFTER UpdateTrackedGeometry because UpdateTrackedGeometry silently assign Tracked state even if tracking lost
	TrackedGeometry->SetTrackingState(TrackingState); 
}