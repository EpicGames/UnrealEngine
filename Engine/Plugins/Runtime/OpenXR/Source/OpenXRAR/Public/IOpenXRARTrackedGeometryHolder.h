// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARTypes.h"
#include "ARSupportInterface.h"

class IOpenXRARTrackedGeometryHolder : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("OpenXRARTrackedGeometryHolder"));
		return FeatureName;
	}

	virtual ~IOpenXRARTrackedGeometryHolder() {}

	// @todo: deprecation - since existing third party plugins already out there use this interface, we'll need to revisit the deprecations. Disabling for now to avoid CIS errors.
//	UE_DEPRECATED(4.27, "Use overload with SharedPtr instead.")		
	virtual void ARTrackedGeometryAdded(struct FOpenXRARTrackedGeometryData* InData) = 0;
//	UE_DEPRECATED(4.27, "Use overload with SharedPtr instead.")
	virtual void ARTrackedGeometryUpdated(struct FOpenXRARTrackedGeometryData* InData) = 0;
//	UE_DEPRECATED(4.27, "Use overload with SharedPtr instead.")
	virtual void ARTrackedGeometryRemoved(struct FOpenXRARTrackedGeometryData* InData) = 0;

	virtual void ARTrackedGeometryAdded(TSharedPtr<struct FOpenXRARTrackedGeometryData> InData) = 0;
	virtual void ARTrackedGeometryUpdated(TSharedPtr<struct FOpenXRARTrackedGeometryData> InData) = 0;
	virtual void ARTrackedGeometryRemoved(TSharedPtr<struct FOpenXRARTrackedGeometryData> InData) = 0;
};

class IOpenXRARTrackedMeshHolder
{
public:
	virtual ~IOpenXRARTrackedMeshHolder() {}

	virtual void StartMeshUpdates() = 0;
	virtual struct FOpenXRMeshUpdate* AllocateMeshUpdate(FGuid InGuidMeshUpdate) = 0;
	virtual void RemoveMesh(FGuid InGuidMeshUpdate) = 0;
	virtual struct FOpenXRPlaneUpdate* AllocatePlaneUpdate(FGuid InGuidPlaneUpdate) = 0;
	virtual void RemovePlane(FGuid InGuidPlaneUpdate) = 0;
	virtual void EndMeshUpdates() = 0;
//	UE_DEPRECATED(4.27, "Use overload with SharedPtr instead.")
	virtual void ObjectUpdated(FOpenXRARTrackedGeometryData* InUpdate) = 0;
	virtual void ObjectUpdated(TSharedPtr<struct FOpenXRARTrackedGeometryData>) = 0;
};

// Base class for ARTrackedGeometryData
// These structs are designed to be constructed by openxr plugin code on any thread and then used by UE4 on the Game Thread.
// All their data members need to be safe to create on any thread.
struct FOpenXRARTrackedGeometryData : public FNoncopyable
{
	enum class EDataType : int8
	{
		Unknown,
		Mesh,
		QRCode,
		Plane
	};
	const EDataType DataType;
	FGuid Id;
	EARTrackingState TrackingState = EARTrackingState::Unknown;
	FTransform LocalToTrackingTransform;

protected:
	// Force derived classes to init the type.
	FOpenXRARTrackedGeometryData(EDataType InDataType) :
		DataType(InDataType)
	{
	}

public:
	virtual ~FOpenXRARTrackedGeometryData() {}

	OPENXRAR_API virtual UARTrackedGeometry* ConstructNewTrackedGeometry(TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface) = 0;
	OPENXRAR_API virtual void UpdateTrackedGeometry(UARTrackedGeometry* TrackedGeometry, TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface) = 0;
};

struct FOpenXRQRCodeData : public FOpenXRARTrackedGeometryData
{

	FVector2D Size = {};
	FString QRCode;
	int32 Version = 0;
	double Timestamp = 0;

	FOpenXRQRCodeData() :
		FOpenXRARTrackedGeometryData(EDataType::QRCode)
	{
	}

	OPENXRAR_API virtual UARTrackedGeometry* ConstructNewTrackedGeometry(TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface) override;
	OPENXRAR_API virtual void UpdateTrackedGeometry(UARTrackedGeometry* TrackedGeometry, TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface) override;
};

struct FOpenXRMeshUpdate : public FOpenXRARTrackedGeometryData
{
	EARObjectClassification Type = EARObjectClassification::NotApplicable;
	EARSpatialMeshUsageFlags SpatialMeshUsageFlags = EARSpatialMeshUsageFlags::NotApplicable;
	TArray<FVector3f> Vertices;
	TArray<MRMESH_INDEX_TYPE> Indices;

	FOpenXRMeshUpdate() :
		FOpenXRARTrackedGeometryData(EDataType::Mesh)
	{
	}

	OPENXRAR_API virtual UARTrackedGeometry* ConstructNewTrackedGeometry(TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface) override;
	OPENXRAR_API virtual void UpdateTrackedGeometry(UARTrackedGeometry* TrackedGeometry, TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface) override;

	OPENXRAR_API bool HasSpatialMeshUsageFlag(const EARSpatialMeshUsageFlags& InFlag)
	{
		return ((int32)SpatialMeshUsageFlags & (int32)InFlag) != 0;
	}
};

struct FOpenXRPlaneUpdate : public FOpenXRARTrackedGeometryData
{
	EARObjectClassification Type = EARObjectClassification::NotApplicable;
	EARSpatialMeshUsageFlags SpatialMeshUsageFlags = EARSpatialMeshUsageFlags::NotApplicable;
	FVector Extent;

	FOpenXRPlaneUpdate() :
		FOpenXRARTrackedGeometryData(EDataType::Plane)
	{
	}

	OPENXRAR_API virtual UARTrackedGeometry* ConstructNewTrackedGeometry(TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface) override;
	OPENXRAR_API virtual void UpdateTrackedGeometry(UARTrackedGeometry* TrackedGeometry, TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSupportInterface) override;

	OPENXRAR_API bool HasSpatialMeshUsageFlag(const EARSpatialMeshUsageFlags& InFlag)
	{
		return ((int32)SpatialMeshUsageFlags & (int32)InFlag) != 0;
	}
};
