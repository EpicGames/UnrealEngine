// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXREditorModule.h"
#include "OpenXRAssetDirectory.h"
#include "OpenXRHMDSettings.h"

#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"

#include "Editor/EditorPerformanceSettings.h"

#define LOCTEXT_NAMESPACE "OpenXR"

void FOpenXREditorModule::StartupModule()
{
	FOpenXRAssetDirectory::LoadForCook();

	UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();
	if (EditorPerformanceSettings->bOverrideMaxViewportRenderingResolution)
	{
		UE_LOGF(LogTemp, Warning, "Existing value for UEditorPerformanceSettings::MaxViewportRenderingResolution will be overriden for OpenXR.");
	}

	UE_LOGF(LogTemp, Log, "OpenXR ignores max viewport resolution in editor to support full HMD resolutions.");
	EditorPerformanceSettings->bOverrideMaxViewportRenderingResolution = true;
	EditorPerformanceSettings->MaxViewportRenderingResolution = 0;

	FPropertyChangedEvent DisabledMaxResolutionEvent(EditorPerformanceSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, MaxViewportRenderingResolution)), EPropertyChangeType::ValueSet);
	EditorPerformanceSettings->PostEditChangeProperty(DisabledMaxResolutionEvent);
}

void FOpenXREditorModule::ShutdownModule()
{
	FOpenXRAssetDirectory::ReleaseAll();

	// unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "OpenXR");
		SettingsModule->UnregisterSettings("Project", "Plugins", "OpenXRHMD");
	}
}

IMPLEMENT_MODULE(FOpenXREditorModule, OpenXREditor);

#undef LOCTEXT_NAMESPACE
