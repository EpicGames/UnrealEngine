// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IOpenXREditorModule.h"
#include "Modules/ModuleInterface.h"

#define OPENXR_EDITOR_MODULE_NAME "OpenXREditor"

class FOpenXREditorModule : public IOpenXREditorModule
{
public:
	FOpenXREditorModule() {};

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
