// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <openxr/openxr.h>

// Uncomment to make intellisense work better.
//#include "/../../../../../Source/ThirdParty/OpenXR/include/openxr/openxr.h"

// This number is in a block that is unused by openxr, and that seemed unlikely to be used soon based on the patterns of usage.
// But if it ever was used by openxr we would have to change it.
// See openxr.h for the list.  This is XR_TYPE_VIEW_CONFIGURATION_VIEW_FOV_EPIC + 50.  This is inside the reserved range for the XR_EPIC_view_configuration_fov extension.
#define XR_TYPE_EPIC 1000059050

typedef enum XrStructureTypeEPIC {
	XR_TYPE_SPACE_ACCELERATION_EPIC = XR_TYPE_EPIC,
	XR_TYPE_RHI_CONTEXT_EPIC,
	XR_STRUCTURE_TYPE_EPIC_MAX_ENUM = 0x7FFFFFFF
} XrStructureTypeEPIC;


//  Extensions

#define XR_EPIC_space_acceleration 1
#define XR_EPIC_space_acceleration_SPEC_VERSION 1
#define XR_EPIC_SPACE_ACCELERATION_NAME "XR_EPIC_space_acceleration"

typedef XrFlags64 XrSpaceAccelerationFlagsEPIC;

// Flag bits for XrSpaceAccelerationFlagsEPIC
static const XrSpaceAccelerationFlagsEPIC XR_SPACE_ACCELERATION_LINEAR_VALID_BIT_EPIC = 0x00000001;
static const XrSpaceAccelerationFlagsEPIC XR_SPACE_ACCELERATION_ANGULAR_VALID_BIT_EPIC = 0x00000002;

typedef struct XrSpaceAccelerationEPIC {
	XrStructureType							type;
	void* XR_MAY_ALIAS						next;
	XrSpaceAccelerationFlagsEPIC			accelerationFlags;
	XrVector3f								linearAcceleration;  // m/s^2
	XrVector3f							angularAcceleration; // rad/s^2
} XrSpaceAccelerationEPIC;

typedef struct XrRHIContextEPIC {
	XrStructureType							type;
	void* XR_MAY_ALIAS						next;
	class IRHICommandContext*				RHIContext;
} XrRHIContextEPIC;
