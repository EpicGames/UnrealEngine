// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HeadMountedDisplayBase.h"
#include "XRRenderBridge.h"
#include "XRSwapChain.h"
#include "OpenXRPlatformRHI.h"

#include <openxr/openxr.h>

class FOpenXRHMD;
class FRHICommandListImmediate;

class FOpenXRRenderBridge : public FXRRenderBridge
{
public:
	FOpenXRRenderBridge(XrInstance InInstance)
		: Instance(InInstance)
		, OpenXRHMD(nullptr)
	{ }

	void SetOpenXRHMD(FOpenXRHMD* InHMD) { OpenXRHMD = InHMD; }
	virtual uint64 GetGraphicsAdapterLuid(XrSystemId InSystem) { return 0; };
	virtual void* GetGraphicsBinding(XrSystemId InSystem) = 0;
	virtual FXRSwapChainPtr CreateSwapchain(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags = ETextureCreateFlags::None) = 0;
	FXRSwapChainPtr CreateSwapchain(XrSession InSession, FRHITexture* Template, ETextureCreateFlags CreateFlags)
	{
		if (!Template)
		{
			return nullptr;
		}

		uint8 UnusedOutFormat = 0;
		return CreateSwapchain(InSession,
			Template->GetFormat(),
			UnusedOutFormat,
			Template->GetSizeX(),
			Template->GetSizeY(),
			Template->GetDesc().Dimension == ETextureDimension::Texture3D ? 1 : Template->GetSizeZ(),
			Template->GetNumMips(),
			Template->GetNumSamples(),
			Template->GetDesc().Dimension,
			Template->GetFlags() | CreateFlags,
			Template->GetClearBinding());
	}

	/** 
	 * Used to wrap OpenXR calls which can submit directly to the graphics queue (xrBeginFrame, xrEndFrame, xrAcquireSwapchainImage, xrReleaseSwapchainImage) and their surrounding logic.
	 * When a submission thread is in use, only that thread is permitted to submit to the graphics queue.
	 * Implmentations for RHIs using a submission thread should override this function to run the provided code on the submission thread via RHIRunOnQueue.
	 * 
	 * Note: This is not necessary for calls in Present/HMDOnFinishRendering_RHIThread because FD3D12Viewport::Present and FVulkanViewport::Present 
	 * both flush the submission thread before calling FRHICustomPresent::Present.
	 */
	virtual void RunOnRHISubmissionThread(TFunction<void()>&& CodeToRun)
	{
		ensure(IsInRenderingThread() || IsInRHIThread());
		CodeToRun();
	};

	/** FRHICustomPresent */
	virtual bool Present(FRHIViewport* Viewport, IRHICommandContext& RHICmdContext, int32& InOutSyncInterval) override;
	virtual bool PresentOnSubmissionThread(int32& InOutSyncInterval) override;

	virtual bool Support10BitSwapchain() const { return false; }

	virtual bool HDRGetMetaDataForStereo(EDisplayOutputFormat& OutDisplayOutputFormat, EDisplayColorGamut& OutDisplayColorGamut, bool& OutbHDRSupported) { return false; }

	virtual void SetSkipRate(uint32 SkipRate) {}

	virtual void HMDOnFinishRendering_RHIThread(IRHICommandContext& RHICmdContext);
	virtual void HMDOnFinishRendering_SubmissionThread();

protected:
	XrInstance Instance;
	FOpenXRHMD* OpenXRHMD;

private:
};

#ifdef XR_USE_GRAPHICS_API_D3D11
FOpenXRRenderBridge* CreateRenderBridge_D3D11(XrInstance InInstance);
#endif
#ifdef XR_USE_GRAPHICS_API_D3D12
FOpenXRRenderBridge* CreateRenderBridge_D3D12(XrInstance InInstance);
#endif
#ifdef XR_USE_GRAPHICS_API_OPENGL_ES
FOpenXRRenderBridge* CreateRenderBridge_OpenGLES(XrInstance InInstance);
#endif
#ifdef XR_USE_GRAPHICS_API_OPENGL
FOpenXRRenderBridge* CreateRenderBridge_OpenGL(XrInstance InInstance);
#endif
#ifdef XR_USE_GRAPHICS_API_VULKAN
FOpenXRRenderBridge* CreateRenderBridge_Vulkan(XrInstance InInstance);
#endif
