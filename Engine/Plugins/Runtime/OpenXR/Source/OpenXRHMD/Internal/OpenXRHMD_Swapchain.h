// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HeadMountedDisplayBase.h"
#include "XRSwapChain.h"
#include "OpenXRPlatformRHI.h"

#include <openxr/openxr.h>

class FOpenXRSwapchain : public FXRSwapChain
{
public:
	FOpenXRSwapchain(TArray<FTextureRHIRef>&& InRHITextureSwapChain, const FTextureRHIRef & InRHITexture, XrSwapchain InHandle);
	virtual ~FOpenXRSwapchain();

	virtual void IncrementSwapChainIndex_RHIThread() override final;
	virtual void WaitCurrentImage_RHIThread(int64 Timeout) override final;
	virtual void ReleaseCurrentImage_RHIThread(IRHICommandContext* RHICmdContext) override final;

	XrSwapchain GetHandle() { return Handle; }
	static XrSwapchain CreateSwapchain(XrSession InSession, uint32 PlatformFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, void* Next = nullptr);
	static uint8 GetNearestSupportedSwapchainFormat(XrSession InSession, uint8 RequestedFormat, TFunction<uint32(uint8)> ToPlatformFormat = nullptr);
	void GetFragmentDensityMaps(TArray<FTextureRHIRef>& OutTextureChain, const bool bIsMobileMultiViewEnabled);

protected:
	XrSwapchain Handle;

	/** Whether the image associated with the swapchain has been acquired. */
	std::atomic<bool> ImageAcquired;

	/** Whether the image associated with the swapchain is ready for being written to. */
	std::atomic<bool> ImageReady;
};

#ifdef XR_USE_GRAPHICS_API_D3D11
FXRSwapChainPtr CreateSwapchain_D3D11(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags = ETextureCreateFlags::None);
#endif
#ifdef XR_USE_GRAPHICS_API_D3D12
FXRSwapChainPtr CreateSwapchain_D3D12(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags = ETextureCreateFlags::None);
#endif
#ifdef XR_USE_GRAPHICS_API_OPENGL
FXRSwapChainPtr CreateSwapchain_OpenGL(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags = ETextureCreateFlags::None);
#endif
#ifdef XR_USE_GRAPHICS_API_OPENGL_ES
FXRSwapChainPtr CreateSwapchain_OpenGLES(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags = ETextureCreateFlags::None);
#endif
#ifdef XR_USE_GRAPHICS_API_VULKAN
FXRSwapChainPtr CreateSwapchain_Vulkan(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags = ETextureCreateFlags::None);
#endif