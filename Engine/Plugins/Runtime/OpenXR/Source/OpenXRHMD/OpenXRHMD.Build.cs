// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class OpenXRHMD : ModuleRules
	{
		public OpenXRHMD(ReadOnlyTargetRules Target) : base(Target)
        {
			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"OpenXR"
				}
				);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"HeadMountedDisplay",
					"XRBase",
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
                    "BuildSettings",
                    "InputCore",
					"RHI",
					"RenderCore",
					"Renderer",
					"RenderCore",
                    "Slate",
                    "SlateCore",
					"AugmentedReality",
					"EngineSettings",
					"DeveloperSettings",
				}
				);

			if (Target.bBuildEditor == true)
            {
				PrivateDependencyModuleNames.Add("EditorFramework");
                PrivateDependencyModuleNames.Add("UnrealEd");
			}

            if (Target.Platform == UnrealTargetPlatform.Win64)
            {
				PrivateDefinitions.Add("OPENXRHMD_HAVE_DX11");
				PrivateDefinitions.Add("OPENXRHMD_HAVE_DX12");

				PublicDependencyModuleNames.AddRange(new string[] {
					"D3D11RHI",
					"D3D12RHI"
				});

				if (!bUsePrecompiled || Target.LinkType == TargetLinkType.Monolithic)
				{
					PublicDependencyModuleNames.AddRange(new string[] {
						"DX11",
						"DX12"
					});
				}
			}

            if (Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Android)
            {
				if (Target.Platform == UnrealTargetPlatform.Win64)
				{
					PrivateDefinitions.Add("OPENXRHMD_HAVE_OPENGL");
				}
			
				if (Target.Platform == UnrealTargetPlatform.Android)
				{
					PrivateDefinitions.Add("OPENXRHMD_HAVE_OPENGLES");
				}

				PublicDependencyModuleNames.Add("OpenGLDrv");

				if (!bUsePrecompiled || Target.LinkType == TargetLinkType.Monolithic)
				{
					PublicDependencyModuleNames.Add("OpenGL");
				}
			}

			if (Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Android  
			    || Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
            {
				PrivateDefinitions.Add("OPENXRHMD_HAVE_VULKAN");

				PublicDependencyModuleNames.Add("VulkanRHI");

				if (!bUsePrecompiled || Target.LinkType == TargetLinkType.Monolithic)
				{
					PublicDependencyModuleNames.Add("Vulkan");
				}
			}
		}
	}
}
