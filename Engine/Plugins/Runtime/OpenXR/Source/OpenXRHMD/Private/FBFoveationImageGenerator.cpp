// Copyright Epic Games, Inc. All Rights Reserved.

#include "FBFoveationImageGenerator.h"
#include "SceneView.h"
#include "OpenXRHMD.h"
#include "OpenXRHMD_Swapchain.h"
#include "RenderGraphBuilder.h"

static TAutoConsoleVariable<int32> CVarOpenXRFBFoveationLevel(
	TEXT("xr.OpenXRFBFoveationLevel"),
	0,
	TEXT("Possible foveation levels as specified by the XrFoveationLevelFB enumeration.\n")
	TEXT("0 = None, 1 = Low , 2 = Medium, 3 = High.\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarOpenXRFBFoveationDynamic(
	TEXT("xr.OpenXRFBFoveationDynamic"),
	false,
	TEXT("Whether dynamically changing foveation based on performance headroom is enabled.\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<float> CVarOpenXRFBFoveationVerticalOffset(
	TEXT("xr.OpenXRFBFoveationVerticalOffset"),
	0,
	TEXT("Desired vertical offset in degrees for the center of the foveation pattern.\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarFBFoveationPreview(
	TEXT("xr.OpenXRFBFoveation.Preview"),
	1,
	TEXT("Whether to include FB foveation in VRS preview overlay. This is currently not implemented.")
	TEXT("0 - off, 1 - on (default)"),
	ECVF_RenderThreadSafe);

FFBFoveationImageGenerator::FFBFoveationImageGenerator(bool bIsFoveationExtensionSupported, XrInstance InInstance, FOpenXRHMD* HMD, bool bMobileMultiViewEnabled)
	: bIsMobileMultiViewEnabled(bMobileMultiViewEnabled)
	, CurrentFrameSwapchainIndex(0)
	, bFoveationExtensionSupported(bIsFoveationExtensionSupported)
{
	if (bFoveationExtensionSupported)
	{
		XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrCreateFoveationProfileFB", (PFN_xrVoidFunction*)&xrCreateFoveationProfileFB));
		XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrUpdateSwapchainFB", (PFN_xrVoidFunction*)&xrUpdateSwapchainFB));
		XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrDestroyFoveationProfileFB", (PFN_xrVoidFunction*)&xrDestroyFoveationProfileFB));

		int32 SanitisedFoveationLevel = FMath::Clamp(CVarOpenXRFBFoveationLevel->GetInt(), XrFoveationLevelFB::XR_FOVEATION_LEVEL_NONE_FB, XrFoveationLevelFB::XR_FOVEATION_LEVEL_HIGH_FB);
		bool bFoveationDynamic = CVarOpenXRFBFoveationDynamic->GetBool();

		FoveationLevel = static_cast<XrFoveationLevelFB>(SanitisedFoveationLevel);
		// Vertical offset is in degrees so it does not need to be clamped to values above 0.
		VerticalOffset = CVarOpenXRFBFoveationVerticalOffset->GetFloat();
		FoveationDynamic = bFoveationDynamic ? XR_FOVEATION_DYNAMIC_LEVEL_ENABLED_FB : XR_FOVEATION_DYNAMIC_DISABLED_FB;
	}

	OpenXRHMD = HMD;
}

FRDGTextureRef FFBFoveationImageGenerator::GetImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage)
{
	if (!bFoveationExtensionSupported || !OpenXRHMD || FoveationImages.IsEmpty() || bGetSoftwareImage)
	{
		return nullptr;
	}
	if (FoveationImages.IsValidIndex(CurrentFrameSwapchainIndex))
	{
		FTextureRHIRef SwapchainTexture = FoveationImages[CurrentFrameSwapchainIndex];
		TRefCountPtr<IPooledRenderTarget> PooledRenderTarget = CreateRenderTarget(SwapchainTexture, *SwapchainTexture->GetName().ToString());

		return GraphBuilder.RegisterExternalTexture(PooledRenderTarget, *SwapchainTexture->GetName().ToString(), ERDGTextureFlags::SkipTracking);
	}
	else
	{
		UE_LOGF(LogHMD, Error, "No valid color swapchain to get foveation images from.");
		return nullptr;
	}
}

void FFBFoveationImageGenerator::PrepareImages(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const FMinimalSceneTextures& SceneTextures, bool bPrepareHardwareImages, bool bPrepareSoftwareImages)
{
	//Not implemented as images are updated in UpdateFoveationImages only
	//when foveation parameters change or when the color swapchain is reallocated.
	return;
}

bool FFBFoveationImageGenerator::IsEnabled() const
{
	if (OpenXRHMD)
	{
		return OpenXRHMD->IsStereoEnabled() && bFoveationExtensionSupported;
	}
	return false;
}

bool FFBFoveationImageGenerator::IsSupportedByView(const FSceneView& View) const
{
	// Only used for XR views
	return IStereoRendering::IsStereoEyeView(View);
}

// This is currently not implemented.
FRDGTextureRef FFBFoveationImageGenerator::GetDebugImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage)
{
	return nullptr;
}

void FFBFoveationImageGenerator::UpdateFoveationImages(bool bReallocatedSwapchain)
{
	if (!OpenXRHMD)
	{
		return;
	}
	int32 SanitisedFoveationLevel = FMath::Clamp(CVarOpenXRFBFoveationLevel->GetInt(), XrFoveationLevelFB::XR_FOVEATION_LEVEL_NONE_FB, XrFoveationLevelFB::XR_FOVEATION_LEVEL_HIGH_FB);
	bool bFoveationDynamic = CVarOpenXRFBFoveationDynamic->GetBool();
	float SanitisedVerticalOffset = CVarOpenXRFBFoveationVerticalOffset->GetFloat();

	//Starting foveation level value outside actual range to force image creation at first update.
	static int32 LastSanitisedFoveationLevel = -1;
	static bool bLastFoveationDynamic = false;
	static float LastSanitisedVerticalOffset = 0.0f;

	bool bUpdateFoveationImages = false;
	if (LastSanitisedFoveationLevel != SanitisedFoveationLevel)
	{
		LastSanitisedFoveationLevel = SanitisedFoveationLevel;
		bUpdateFoveationImages = true;
	}
	if (bLastFoveationDynamic != bFoveationDynamic)
	{
		bLastFoveationDynamic = bFoveationDynamic;
		bUpdateFoveationImages = true;
	}
	if (LastSanitisedVerticalOffset != SanitisedVerticalOffset)
	{
		LastSanitisedVerticalOffset = SanitisedVerticalOffset;
		bUpdateFoveationImages = true;
	}
	
	// If the swapchain has been reallocated we need to update the foveation images even if the foveation params haven't changed.
	bUpdateFoveationImages |= bReallocatedSwapchain;

	if (!bUpdateFoveationImages)
	{
		return;
	}

	FoveationLevel = static_cast<XrFoveationLevelFB>(SanitisedFoveationLevel);
	VerticalOffset = SanitisedVerticalOffset;
	FoveationDynamic = bFoveationDynamic ? XR_FOVEATION_DYNAMIC_LEVEL_ENABLED_FB : XR_FOVEATION_DYNAMIC_DISABLED_FB;

	XrFoveationLevelProfileCreateInfoFB FoveationLevelProfileInfo{ XR_TYPE_FOVEATION_LEVEL_PROFILE_CREATE_INFO_FB };
	FoveationLevelProfileInfo.next = nullptr;
	FoveationLevelProfileInfo.level = FoveationLevel;
	FoveationLevelProfileInfo.verticalOffset = VerticalOffset;
	FoveationLevelProfileInfo.dynamic = FoveationDynamic;

	XrFoveationProfileCreateInfoFB FoveationCreateInfo{ XR_TYPE_FOVEATION_PROFILE_CREATE_INFO_FB };
	FoveationCreateInfo.next = &FoveationLevelProfileInfo;

	XrFoveationProfileFB FoveationProfile = XR_NULL_HANDLE;

	XrSession Session = OpenXRHMD->GetSession();
	FOpenXRSwapchain* ColorSwapchain = OpenXRHMD->GetColorSwapchain_RenderThread();

	if (Session && ColorSwapchain != nullptr)
	{
		XR_ENSURE(xrCreateFoveationProfileFB(Session, &FoveationCreateInfo, &FoveationProfile));

		XrSwapchainStateFoveationFB SwapchainFoveationState{ XR_TYPE_SWAPCHAIN_STATE_FOVEATION_FB };
		SwapchainFoveationState.flags = 0; // As per OpenXR specification.
		SwapchainFoveationState.next = nullptr;
		SwapchainFoveationState.profile = FoveationProfile;

		XR_ENSURE(xrUpdateSwapchainFB(ColorSwapchain->GetHandle(), reinterpret_cast<const XrSwapchainStateBaseHeaderFB*>(&SwapchainFoveationState)));

		XR_ENSURE(xrDestroyFoveationProfileFB(FoveationProfile));

		ColorSwapchain->GetFragmentDensityMaps(FoveationImages, bIsMobileMultiViewEnabled);
	}
}

void FFBFoveationImageGenerator::SetCurrentFrameSwapchainIndex(int32 InCurrentFrameSwapchainIndex)
{
	CurrentFrameSwapchainIndex = InCurrentFrameSwapchainIndex;
}