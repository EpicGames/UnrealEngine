// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VariableRateShadingImageManager.h"

#include <openxr/openxr.h>

class FRGDBuilder;
class FOpenXRHMD;
class FSceneView;

class FFBFoveationImageGenerator : public IVariableRateShadingImageGenerator
{
public:
	FFBFoveationImageGenerator(bool bIsFoveationExtensionSupported, XrInstance InInstance, FOpenXRHMD* HMD, bool bMobileMultiViewEnabled);
	virtual ~FFBFoveationImageGenerator() override {};

	bool IsFoveationExtensionEnabled() { return bFoveationExtensionSupported;}
	// bReallocatedSwapchain forces foveation images to update even if the foveation params haven't changed.
	void UpdateFoveationImages(bool bReallocatedSwapchain = false);
	void SetCurrentFrameSwapchainIndex(int32 CurrentFrameSwapchainIndex);

	/** IVariableRateShadingImageGenerator interface */
	virtual FRDGTextureRef GetImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage = false) override;
	virtual void PrepareImages(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const FMinimalSceneTextures& SceneTextures, bool bPrepareHardwareImages, bool bPrepareSoftwareImages) override;
	virtual bool IsEnabled() const override;
	virtual bool IsSupportedByView(const FSceneView& View) const override;
	virtual FRDGTextureRef GetDebugImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage = false) override;
	virtual FVariableRateShadingImageManager::EVRSSourceType GetType() const override
	{
		return FVariableRateShadingImageManager::EVRSSourceType::FixedFoveation;
	}
	/** End of IVariableRateShadingImageGenerator interface */

private:

	FOpenXRHMD* OpenXRHMD;
	bool		bIsMobileMultiViewEnabled;
	int32		CurrentFrameSwapchainIndex;

	// XR_FB_foveation
	bool					bFoveationExtensionSupported;
	XrFoveationLevelFB		FoveationLevel;
	float					VerticalOffset;
	XrFoveationDynamicFB	FoveationDynamic;
	TArray<FTextureRHIRef>	FoveationImages;
	PFN_xrCreateFoveationProfileFB	xrCreateFoveationProfileFB;
	PFN_xrUpdateSwapchainFB			xrUpdateSwapchainFB;
	PFN_xrDestroyFoveationProfileFB xrDestroyFoveationProfileFB;
};

