// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRAssetManager.h"
#include "OpenXRHMD.h"
#include "OpenXRCore.h"
#include "IOpenXRExtensionPlugin.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "UObject/SoftObjectPath.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "OpenXRAssetDirectory.h"
#include "UObject/GCObject.h"
#include "UObject/SoftObjectPtr.h"

/* FOpenXRAssetDirectory
 *****************************************************************************/

FSoftObjectPath FOpenXRAssetDirectory::HPMixedRealityLeft         = FString(TEXT("/OpenXR/Devices/HPMixedReality/Left/left_HPMixedRealityController.left_HPMixedRealityController"));
FSoftObjectPath FOpenXRAssetDirectory::HPMixedRealityRight        = FString(TEXT("/OpenXR/Devices/HPMixedReality/Right/right_HPMixedRealityController.right_HPMixedRealityController"));
FSoftObjectPath FOpenXRAssetDirectory::HTCVive                    = FString(TEXT("/OpenXR/Devices/HTCVive/HTCViveController.HTCViveController"));
FSoftObjectPath FOpenXRAssetDirectory::HTCViveCosmosLeft          = FString(TEXT("/OpenXR/Devices/HTCViveCosmos/Left/left_HTCViveCosmosController.left_HTCViveCosmosController"));
FSoftObjectPath FOpenXRAssetDirectory::HTCViveCosmosRight         = FString(TEXT("/OpenXR/Devices/HTCViveCosmos/Right/right_HTCViveCosmosController.right_HTCViveCosmosController"));
FSoftObjectPath FOpenXRAssetDirectory::HTCViveFocus               = FString(TEXT("/OpenXR/Devices/HTCViveFocus/HTCViveFocusController.HTCViveFocusController"));
FSoftObjectPath FOpenXRAssetDirectory::HTCViveFocusPlus           = FString(TEXT("/OpenXR/Devices/HTCViveFocusPlus/HTCViveFocusPlusController.HTCViveFocusPlusController"));
FSoftObjectPath FOpenXRAssetDirectory::MicrosoftMixedRealityLeft  = FString(TEXT("/OpenXR/Devices/MicrosoftMixedReality/Left/left_MicrosoftMixedRealityController.left_MicrosoftMixedRealityController"));
FSoftObjectPath FOpenXRAssetDirectory::MicrosoftMixedRealityRight = FString(TEXT("/OpenXR/Devices/MicrosoftMixedReality/Right/right_MicrosoftMixedRealityController.right_MicrosoftMixedRealityController"));
FSoftObjectPath FOpenXRAssetDirectory::OculusGo                   = FString(TEXT("/OpenXR/Devices/OculusGo/OculusGoController.OculusGoController"));
FSoftObjectPath FOpenXRAssetDirectory::OculusTouchLeft            = FString(TEXT("/OpenXR/Devices/OculusTouch/Left/left_OculusTouchController.left_OculusTouchController"));
FSoftObjectPath FOpenXRAssetDirectory::OculusTouchRight           = FString(TEXT("/OpenXR/Devices/OculusTouch/Right/right_OculusTouchController.right_OculusTouchController"));
FSoftObjectPath FOpenXRAssetDirectory::OculusTouchV2Left          = FString(TEXT("/OpenXR/Devices/OculusTouch_v2/Left/left_OculusTouch_v2Controller.left_OculusTouch_v2Controller"));
FSoftObjectPath FOpenXRAssetDirectory::OculusTouchV2Right         = FString(TEXT("/OpenXR/Devices/OculusTouch_v2/Right/right_OculusTouch_v2Controller.right_OculusTouch_v2Controller"));
FSoftObjectPath FOpenXRAssetDirectory::OculusTouchV3Left          = FString(TEXT("/OpenXR/Devices/OculusTouch_v3/Left/left_OculusTouch_v3Controller.left_OculusTouch_v3Controller"));
FSoftObjectPath FOpenXRAssetDirectory::OculusTouchV3Right         = FString(TEXT("/OpenXR/Devices/OculusTouch_v3/Right/right_OculusTouch_v3Controller.right_OculusTouch_v3Controller"));
FSoftObjectPath FOpenXRAssetDirectory::PicoNeo3Left				  = FString(TEXT("/OpenXR/Devices/PicoNeo3/Left/left_PicoNeo3Controller.left_PicoNeo3Controller"));
FSoftObjectPath FOpenXRAssetDirectory::PicoNeo3Right              = FString(TEXT("/OpenXR/Devices/PicoNeo3/Right/right_PicoNeo3Controller.right_PicoNeo3Controller"));
FSoftObjectPath FOpenXRAssetDirectory::SamsungGearVR              = FString(TEXT("/OpenXR/Devices/SamsungGearVR/SamsungGearVRController.SamsungGearVRController"));
FSoftObjectPath FOpenXRAssetDirectory::SamsungOdysseyLeft         = FString(TEXT("/OpenXR/Devices/SamsungOdyssey/Left/left_SamsungOdysseyController.left_SamsungOdysseyController"));
FSoftObjectPath FOpenXRAssetDirectory::SamsungOdysseyRight        = FString(TEXT("/OpenXR/Devices/SamsungOdyssey/Right/right_SamsungOdysseyController.right_SamsungOdysseyController"));
FSoftObjectPath FOpenXRAssetDirectory::ValveIndexLeft             = FString(TEXT("/OpenXR/Devices/ValveIndex/Left/left_ValveIndexController.left_ValveIndexController"));
FSoftObjectPath FOpenXRAssetDirectory::ValveIndexRight            = FString(TEXT("/OpenXR/Devices/ValveIndex/Right/right_ValveIndexController.right_ValveIndexController"));


/* OpenXRAssetManager_Impl
 *****************************************************************************/

namespace OpenXRAssetManager_Impl
{
	struct FRenderableDevice
	{
		const char* UserPath;
		const char* InteractionProfile;
		FSoftObjectPath MeshAssetRef;
	};

	static TArray<FRenderableDevice> RenderableDevices =
	{
		{ "/user/hand/left",  "/interaction_profiles/htc/vive_controller", FOpenXRAssetDirectory::HTCVive },
		{ "/user/hand/right", "/interaction_profiles/htc/vive_controller", FOpenXRAssetDirectory::HTCVive },
		{ "/user/hand/left",  "/interaction_profiles/microsoft/motion_controller", FOpenXRAssetDirectory::MicrosoftMixedRealityLeft },
		{ "/user/hand/right", "/interaction_profiles/microsoft/motion_controller", FOpenXRAssetDirectory::MicrosoftMixedRealityRight },
		{ "/user/hand/left",  "/interaction_profiles/oculus/go_controller", FOpenXRAssetDirectory::OculusGo },
		{ "/user/hand/right", "/interaction_profiles/oculus/go_controller", FOpenXRAssetDirectory::OculusGo },
		{ "/user/hand/left",  "/interaction_profiles/oculus/touch_controller", FOpenXRAssetDirectory::OculusTouchLeft },
		{ "/user/hand/right", "/interaction_profiles/oculus/touch_controller", FOpenXRAssetDirectory::OculusTouchRight },
		{ "/user/hand/left",  "/interaction_profiles/valve/index_controller", FOpenXRAssetDirectory::ValveIndexLeft },
		{ "/user/hand/right", "/interaction_profiles/valve/index_controller", FOpenXRAssetDirectory::ValveIndexRight },
	};
};

#if WITH_EDITORONLY_DATA
class FOpenXRAssetRepo : public FGCObject, public TArray<TObjectPtr<UObject>>
{
public:
	// made an on-demand singleton rather than a static global, to avoid issues with FGCObject initialization
	static FOpenXRAssetRepo& Get()
	{
		static FOpenXRAssetRepo AssetRepository;
		return AssetRepository;
	}

	UObject* LoadAndAdd(const FSoftObjectPath& AssetPath)
	{
		UObject* AssetObj = AssetPath.TryLoad();
		if (AssetObj != nullptr)
		{
			AddUnique(AssetObj);
		}
		return AssetObj;
	}

public:
	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObjects(*this);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FOpenXRAssetRepo");
	}
};

void FOpenXRAssetDirectory::LoadForCook()
{
	FOpenXRAssetRepo& AssetRepo = FOpenXRAssetRepo::Get();
	for (const OpenXRAssetManager_Impl::FRenderableDevice& RenderableDevice : OpenXRAssetManager_Impl::RenderableDevices)
	{
		AssetRepo.LoadAndAdd(RenderableDevice.MeshAssetRef);
	}

	// Query all extension plugins for controller models
	TArray<IOpenXRExtensionPlugin*> ExtModules = IModularFeatures::Get().GetModularFeatureImplementations<IOpenXRExtensionPlugin>(IOpenXRExtensionPlugin::GetModularFeatureName());
	for (IOpenXRExtensionPlugin* Plugin : ExtModules)
	{
		TArray<FSoftObjectPath> Paths;
		Plugin->GetControllerModelsForCooking(Paths);
		for (const FSoftObjectPath& AssetPath : Paths)
		{
			AssetRepo.LoadAndAdd(AssetPath);
		}
	}

	// These meshes are handled explicitly based on the system name
	AssetRepo.LoadAndAdd(FOpenXRAssetDirectory::OculusTouchV2Left);
	AssetRepo.LoadAndAdd(FOpenXRAssetDirectory::OculusTouchV2Right);
	AssetRepo.LoadAndAdd(FOpenXRAssetDirectory::OculusTouchV3Left);
	AssetRepo.LoadAndAdd(FOpenXRAssetDirectory::OculusTouchV3Right);
}

void FOpenXRAssetDirectory::ReleaseAll()
{
	FOpenXRAssetRepo::Get().Empty();
}
#endif // WITH_EDITORONLY_DATA


/* FOpenXRAssetManager
*****************************************************************************/

FOpenXRAssetManager::FOpenXRAssetManager(XrInstance Instance, FOpenXRHMD* InHMD)
	: OpenXRHMD(InHMD)
	, Quest1(TEXT("Oculus Quest"))
	, Quest2(TEXT("Oculus Quest2"))
{
	IModularFeatures::Get().RegisterModularFeature(IXRSystemAssets::GetModularFeatureName(), this);

	XR_ENSURE(xrStringToPath(Instance, "/user/hand/left", &LeftHand));
	XR_ENSURE(xrStringToPath(Instance, "/user/hand/right", &RightHand));

	for (const OpenXRAssetManager_Impl::FRenderableDevice& RenderableDevice : OpenXRAssetManager_Impl::RenderableDevices)
	{
		TPair<XrPath, XrPath> ProfileDevicePair;
		XR_ENSURE(xrStringToPath(Instance, RenderableDevice.InteractionProfile, &ProfileDevicePair.Key));
		XR_ENSURE(xrStringToPath(Instance, RenderableDevice.UserPath, &ProfileDevicePair.Value));
		
		DeviceMeshes.Add(ProfileDevicePair, RenderableDevice.MeshAssetRef);
	}

	// These meshes are handled explicitly based on the system name
	Quest1Meshes.Add(LeftHand, FOpenXRAssetDirectory::OculusTouchV2Left);
	Quest1Meshes.Add(RightHand, FOpenXRAssetDirectory::OculusTouchV2Right);
	Quest2Meshes.Add(LeftHand, FOpenXRAssetDirectory::OculusTouchV3Left);
	Quest2Meshes.Add(RightHand, FOpenXRAssetDirectory::OculusTouchV3Right);
}

FOpenXRAssetManager::~FOpenXRAssetManager()
{
	IModularFeatures::Get().UnregisterModularFeature(IXRSystemAssets::GetModularFeatureName(), this);
}

bool FOpenXRAssetManager::EnumerateRenderableDevices(TArray<int32>& DeviceListOut)
{
	return OpenXRHMD->EnumerateTrackedDevices(DeviceListOut, EXRTrackedDeviceType::Controller);
}

int32 FOpenXRAssetManager::GetDeviceId(EControllerHand ControllerHand)
{
	XrPath Target = (ControllerHand == EControllerHand::Right) ? RightHand : LeftHand;

	TArray<int32> DeviceList;
	if (OpenXRHMD->EnumerateTrackedDevices(DeviceList, EXRTrackedDeviceType::Controller))
	{
		if (ControllerHand == EControllerHand::AnyHand)
		{
			return DeviceList[0];
		}

		for (int32 i = 0; i < DeviceList.Num(); i++)
		{
			if (OpenXRHMD->GetTrackedDevicePath(i) == Target)
			{
				return i;
			}
		}
	}
	return INDEX_NONE;
}

UPrimitiveComponent* FOpenXRAssetManager::CreateRenderComponent(const int32 DeviceId, AActor* Owner, EObjectFlags Flags, const bool /*bForceSynchronous*/, const FXRComponentLoadComplete& OnLoadComplete)
{
	UPrimitiveComponent* NewRenderComponent = nullptr;

	XrInstance Instance = OpenXRHMD->GetInstance();
	XrSession Session = OpenXRHMD->GetSession();
	XrPath DevicePath = OpenXRHMD->GetTrackedDevicePath(DeviceId);
	if (Session && DevicePath && OpenXRHMD->IsRunning())
	{
		XrInteractionProfileState Profile;
		Profile.type = XR_TYPE_INTERACTION_PROFILE_STATE;
		Profile.next = nullptr;
		if (XR_FAILED(xrGetCurrentInteractionProfile(Session, DevicePath, &Profile)))
		{
			// This call can often fail because action sets are not yet attached to the session
			// or the user path doesn't have an interaction profile assigned yet.
			return nullptr;
		}

		FSoftObjectPath DeviceMeshPath;
		for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
		{
			if (Plugin->GetControllerModel(Instance, Profile.interactionProfile, DevicePath, DeviceMeshPath))
			{
				break;
			}

			// Reset the path in case the failed query assigned it
			DeviceMeshPath.Reset();
		}

		if (DeviceMeshPath.IsNull())
		{
			FSoftObjectPath* DeviceMeshPtr = nullptr;
			FName System = OpenXRHMD->GetHMDName();
			if (System == Quest1)
			{
				DeviceMeshPtr = Quest1Meshes.Find(DevicePath);
			}
			else if (System == Quest2)
			{
				DeviceMeshPtr = Quest2Meshes.Find(DevicePath);
			}
			else
			{
				TPair<XrPath, XrPath> Key(Profile.interactionProfile, DevicePath);
				DeviceMeshPtr = DeviceMeshes.Find(Key);
			}

			if (!DeviceMeshPtr)
			{
				return nullptr;
			}
			DeviceMeshPath = *DeviceMeshPtr;
		}

		if (UObject* DeviceMesh = DeviceMeshPath.TryLoad())
		{
			if (UStaticMesh* AsStaticMesh = Cast<UStaticMesh>(DeviceMesh))
			{
				const FName ComponentName = MakeUniqueObjectName(Owner, UStaticMeshComponent::StaticClass(), *FString::Printf(TEXT("%s_Device%d"), TEXT("OpenXR"), DeviceId));
				UStaticMeshComponent* MeshComponent = NewObject<UStaticMeshComponent>(Owner, ComponentName, Flags);

				MeshComponent->SetStaticMesh(AsStaticMesh);
				NewRenderComponent = MeshComponent;
			}
		}
	}

	OnLoadComplete.ExecuteIfBound(NewRenderComponent);
	return NewRenderComponent;
}
