// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenXRCore.h"
#include "IXRSystemAssets.h"
#include "UObject/SoftObjectPtr.h"

class FOpenXRHMD;
class UStaticMesh;

/**
 *
 */
class FOpenXRAssetManager : public IXRSystemAssets
{
public:
	FOpenXRAssetManager(XrInstance Instance, FOpenXRHMD* InHMD);
	virtual ~FOpenXRAssetManager();

public:
	//~ IXRSystemAssets interface 

	virtual bool EnumerateRenderableDevices(TArray<int32>& DeviceListOut) override;
	virtual int32 GetDeviceId(EControllerHand ControllerHand) override;
	virtual UPrimitiveComponent* CreateRenderComponent(const int32 DeviceId, AActor* Owner, EObjectFlags Flags, const bool bForceSynchronous, const FXRComponentLoadComplete& OnLoadComplete) override;

private:
	FOpenXRHMD* OpenXRHMD;

	XrPath LeftHand;
	XrPath RightHand;
	TMap<TPair<XrPath, XrPath>, FSoftObjectPath> DeviceMeshes;

	// Oculus Quest platforms use different Touch controllers, but share the same interaction profile
	FName Quest1, Quest2;
	TMap<XrPath, FSoftObjectPath> Quest1Meshes, Quest2Meshes;
};