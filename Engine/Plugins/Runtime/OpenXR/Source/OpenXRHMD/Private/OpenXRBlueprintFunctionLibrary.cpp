// Copyright Epic Games, Inc. All Rights Reserved.

#include <OpenXRBlueprintFunctionLibrary.h>
#include <OpenXRHMD.h>

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenXRBlueprintFunctionLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogUOpenXRBlueprintFunctionLibrary, Log, All);

UOpenXRBlueprintFunctionLibrary::UOpenXRBlueprintFunctionLibrary(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UOpenXRBlueprintFunctionLibrary::SetEnvironmentBlendMode(EOpenXREnvironmentBlendMode NewBlendMode) 
{
	if (NewBlendMode == EOpenXREnvironmentBlendMode::None)
	{
		UE_LOGF(LogUOpenXRBlueprintFunctionLibrary, Warning, "Attempted to set environment blend mode to EOpenXREnvironmentBlendMode::None. No blend mode will be set and the runtime will continue to use its current blend mode.");
		return;
	}

	FOpenXRHMD* OpenXRHMD = GetOpenXRHMD();
	if (OpenXRHMD) 
	{
		OpenXRHMD->SetEnvironmentBlendMode(static_cast<XrEnvironmentBlendMode>(NewBlendMode));
	}
}

EOpenXREnvironmentBlendMode UOpenXRBlueprintFunctionLibrary::GetEnvironmentBlendMode() 
{
	XrEnvironmentBlendMode CurrentBlendMode = XR_ENVIRONMENT_BLEND_MODE_MAX_ENUM;

	FOpenXRHMD* OpenXRHMD = GetOpenXRHMD();
	if (OpenXRHMD) 
	{
		CurrentBlendMode = OpenXRHMD->GetEnvironmentBlendMode();
	}

	switch (CurrentBlendMode)
	{
	case XR_ENVIRONMENT_BLEND_MODE_OPAQUE:
	case XR_ENVIRONMENT_BLEND_MODE_ADDITIVE:
	case XR_ENVIRONMENT_BLEND_MODE_ALPHA_BLEND:
		return static_cast<EOpenXREnvironmentBlendMode>(CurrentBlendMode);

	case XR_ENVIRONMENT_BLEND_MODE_MAX_ENUM:
		return EOpenXREnvironmentBlendMode::None; // HMD is uninitialized

	default:
		checkNoEntry();
		return EOpenXREnvironmentBlendMode::None;
	}
}

TArray<EOpenXREnvironmentBlendMode> UOpenXRBlueprintFunctionLibrary::GetSupportedEnvironmentBlendModes() 
{
	TArray<XrEnvironmentBlendMode> SupportedBlendModes = {};
	TArray<EOpenXREnvironmentBlendMode> OutputSupportedBlendModes = {};

	FOpenXRHMD* OpenXRHMD = GetOpenXRHMD();
	if (OpenXRHMD) 
	{
		SupportedBlendModes = OpenXRHMD->GetSupportedEnvironmentBlendModes(); // These should be ordered from highest to lowest runtime preference
	}
	
	for (XrEnvironmentBlendMode BlendMode : SupportedBlendModes)
	{
		switch (BlendMode)
		{
		case XR_ENVIRONMENT_BLEND_MODE_OPAQUE:
		case XR_ENVIRONMENT_BLEND_MODE_ADDITIVE:
		case XR_ENVIRONMENT_BLEND_MODE_ALPHA_BLEND:
			OutputSupportedBlendModes.Add(static_cast<EOpenXREnvironmentBlendMode>(BlendMode));
			break;

		case XR_ENVIRONMENT_BLEND_MODE_MAX_ENUM:
		default:
			checkNoEntry();
			break;
		}
	}
	
	return OutputSupportedBlendModes;
}

bool UOpenXRBlueprintFunctionLibrary::IsCompositionLayerInvertedAlphaEnabled() 
{
	FOpenXRHMD* OpenXRHMD = GetOpenXRHMD();
	if (OpenXRHMD) 
	{
		static const TConsoleVariableData<bool>* OpenXRInvertAlphaCVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("xr.OpenXRInvertAlpha"));
		const bool bEXTExtensionEnabled = OpenXRHMD->IsExtensionEnabled(XR_EXT_COMPOSITION_LAYER_INVERTED_ALPHA_EXTENSION_NAME);
		const bool bFBExtensionEnabled = OpenXRHMD->IsExtensionEnabled(XR_FB_COMPOSITION_LAYER_ALPHA_BLEND_EXTENSION_NAME);
		
		return OpenXRInvertAlphaCVar->GetValueOnGameThread() && (bEXTExtensionEnabled || bFBExtensionEnabled);
	}

	return false;
}

FOpenXRHMD* UOpenXRBlueprintFunctionLibrary::GetOpenXRHMD()
{
	static FName SystemName(TEXT("OpenXR"));
	if (GEngine->XRSystem.IsValid() && (GEngine->XRSystem->GetSystemName() == SystemName))
	{
		return static_cast<FOpenXRHMD*>(GEngine->XRSystem.Get());
	}

	return nullptr;
}
