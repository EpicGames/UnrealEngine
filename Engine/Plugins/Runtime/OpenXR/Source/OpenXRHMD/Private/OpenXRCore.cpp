// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRCore.h"
#include "HeadMountedDisplayTypes.h"
#include "IOpenXRHMDModule.h"

FOpenXRPath::FOpenXRPath(XrPath InPath)
	: Path(InPath)
{
}

FOpenXRPath::FOpenXRPath(FName InName)
	: Path(IOpenXRHMDModule::Get().ResolveNameToPath(InName))
{
}

FOpenXRPath::FOpenXRPath(const char* PathString)
	: Path(XR_NULL_PATH)
{
	XrInstance Instance = IOpenXRHMDModule::Get().GetInstance();
	if (ensure(Instance != XR_NULL_HANDLE))
	{
		XR_ENSURE(xrStringToPath(Instance, PathString, &Path));
	}
}

FOpenXRPath::FOpenXRPath(const FString& PathString)
	: FOpenXRPath((ANSICHAR*)StringCast<ANSICHAR>(*PathString).Get())
{
}

FString FOpenXRPath::ToString() const
{
	XrInstance Instance = IOpenXRHMDModule::Get().GetInstance();
	if (!ensure(Instance != XR_NULL_HANDLE))
	{
		return FString();
	}

	uint32 PathCount = 0;
	char PathChars[XR_MAX_PATH_LENGTH];
	XrResult Result = xrPathToString(Instance, Path, XR_MAX_PATH_LENGTH, &PathCount, PathChars);
	ensure(XR_SUCCEEDED(Result));
	return XR_SUCCEEDED(Result) ? FString::ConstructFromPtrSize(PathChars, PathCount - 1) : FString();
}

uint32 FOpenXRPath::GetStringLength() const
{
	XrInstance Instance = IOpenXRHMDModule::Get().GetInstance();
	if (!ensure(Instance != XR_NULL_HANDLE))
	{
		return 0;
	}

	uint32 PathCount = 0;
	XrResult Result = xrPathToString(Instance, Path, 0, &PathCount, nullptr);
	check(XR_SUCCEEDED(Result));
	return PathCount - 1;
}

FName FOpenXRPath::ToName() const
{
	return IOpenXRHMDModule::Get().ResolvePathToName(Path);
}

#define DEFINE_XR_ENTRYPOINTS(Type,Func) Type OpenXRDynamicAPI::Func = nullptr;
ENUM_XR_ENTRYPOINTS_GLOBAL(DEFINE_XR_ENTRYPOINTS);
ENUM_XR_ENTRYPOINTS(DEFINE_XR_ENTRYPOINTS);
DEFINE_XR_ENTRYPOINTS(PFN_xrGetInstanceProcAddr,xrGetInstanceProcAddr);

#define GET_XR_ENTRYPOINTS(Type,Func) if (!XR_ENSURE(xrGetInstanceProcAddr(Instance, #Func, (PFN_xrVoidFunction*)&Func))) \
		{ UE_LOGF(LogHMD, Warning, "Failed to find entry point for %ls", TEXT(#Func)); return false; }

bool PreInitOpenXRCore(PFN_xrGetInstanceProcAddr InGetProcAddr)
{
	if (!InGetProcAddr)
	{
		return false;
	}
	xrGetInstanceProcAddr = InGetProcAddr;

	XrInstance Instance = XR_NULL_HANDLE;
	ENUM_XR_ENTRYPOINTS_GLOBAL(GET_XR_ENTRYPOINTS);
	return true;
}

bool InitOpenXRCore(XrInstance Instance)
{
	if (!Instance)
	{
		return false;
	}

	ENUM_XR_ENTRYPOINTS(GET_XR_ENTRYPOINTS);
	return true;
}

void EnumerateOpenXRApiLayers(TArray<XrApiLayerProperties>& OutProperties)
{
	uint32_t Count = 0;
	XR_ENSURE(xrEnumerateApiLayerProperties(0, &Count, nullptr));
	OutProperties.SetNum(Count);
	for (auto& Prop : OutProperties)
	{
		Prop = XrApiLayerProperties{ XR_TYPE_API_LAYER_PROPERTIES };
	}
	XR_ENSURE(xrEnumerateApiLayerProperties(Count, &Count, OutProperties.GetData()));
}

#undef GET_XR_ENTRYPOINTS
