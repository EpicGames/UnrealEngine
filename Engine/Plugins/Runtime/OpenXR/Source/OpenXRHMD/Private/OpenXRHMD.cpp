// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRHMD.h"

#include "AnalyticsEventAttribute.h"
#include "ClearQuad.h"
#include "CommonRenderResources.h"
#include "Containers/ArrayView.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DefaultStereoLayers.h"
#include "Engine/Engine.h"
#include "Engine/GameEngine.h"
#include "Epic_openxr.h"
#include "FBFoveationImageGenerator.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/WorldSettings.h"
#include "IHandTracker.h"
#include "IOpenXRExtensionPlugin.h"
#include "IOpenXRHMDModule.h"
#include "Misc/App.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Parse.h"
#include "Modules/ModuleManager.h"
#include "OpenXRCore.h"
#include "OpenXRHMDSettings.h"
#include "OpenXRHMD_Layer.h"
#include "OpenXRHMD_RenderBridge.h"
#include "OpenXRHMD_Swapchain.h"
#include "PixelShaderUtils.h"
#include "RenderUtils.h"
#include "SceneView.h"
#include "Shader.h"
#include "Slate/SceneViewport.h"
#include "Slate/SlateViewportProvider.h"
#include "StereoRenderUtils.h"
#include "VariableRateShadingImageManager.h"

#if PLATFORM_ANDROID
#include "Android/AndroidApplication.h"
#endif

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Misc/MessageDialog.h"
#include "UnrealEdMisc.h"
#endif

static const TCHAR* HMDThreadString()
{
	if (IsInGameThread())
	{
		return TEXT("T~G");
	}
	else if (IsInRenderingThread())
	{
		return TEXT("T~R");
	}
	else if (IsInRHIThread())
	{
		return TEXT("T~I");
	}
	else
	{
		return TEXT("T~?");
	}
}

#define LOCTEXT_NAMESPACE "OpenXR"

static const int64 OPENXR_SWAPCHAIN_WAIT_TIMEOUT = 100000000ll; // 100ms in nanoseconds.

namespace UE::OpenXR
{
	static bool bStandAloneStereoOnlyDevice = false;
	static FAutoConsoleVariableRef CVarStandAloneStereoOnlyDevice(
		TEXT("XR.StandAloneStereoOnlyDevice"),
		bStandAloneStereoOnlyDevice,
		TEXT("If true, the current device will be considered stereo only, meaning it is a dedicated VR headset.  This is very often set in BaseDeviceProfiles.ini\n"),
		ECVF_Default | ECVF_ReadOnly);

	// This approach, albeit OpenXR compliant, uses one pass per stereo layer. Therefore, performance might be affected,
	// specially as the number of layers increases. This could / should potentially be improved in the future with a single ordered pass with
	// a custom depth test to apply depth to only layers that have it
	static bool bRenderEmulatedLayerDepth = true;
	static FAutoConsoleVariableRef CVarRenderEmulatedLayerDepth(
		TEXT("XR.StereoLayers.RenderEmulatedLayerDepth"),
		bRenderEmulatedLayerDepth,
		TEXT("If true, emulated stereo layers will be rendered to be fully compliant with OpenXR and support depth.\n"),
		ECVF_Default);
}

static TAutoConsoleVariable<int32> CVarOpenXRPausedIdleFPS(
	TEXT("xr.OpenXRPausedIdleFPS"),
	10,
	TEXT("If non-zero MaxFPS will be set to this value when the XRSession state is XR_SESSION_STATE_IDLE, which often means the HMD has been removed from the users head.\n")
	TEXT("Defaults to 10fps. 0 will allow unreal to run as fast as it can.  Note that in XR_SESSION_STATE_IDLE the frame rate may actually be higher than when in VR, so you may want to set it to 60 or 90 rather than 0.\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarOpenXRExitAppOnRuntimeDrivenSessionExit(
	TEXT("xr.OpenXRExitAppOnRuntimeDrivenSessionExit"),
	1,
	TEXT("If true, RequestExitApp will be called after we destroy the session because the state transitioned to XR_SESSION_STATE_EXITING or XR_SESSION_STATE_LOSS_PENDING and this is NOT the result of a call from the App to xrRequestExitSession.\n")
	TEXT("The aniticipated situation is that the runtime is associated with a launcher application or has a runtime UI overlay which can tell openxr to exit vr and that in that context the app should also exit.  But maybe there are cases where it should not?  Set this CVAR to make it not.\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarOpenXREnvironmentBlendMode(
	TEXT("xr.OpenXREnvironmentBlendMode"),
	0,
	TEXT("Override the XrEnvironmentBlendMode used when submitting frames. 1 = Opaque, 2 = Additive, 3 = Alpha Blend\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarOpenXRForceStereoLayerEmulation(
	TEXT("xr.OpenXRForceStereoLayerEmulation"),
	false,
	TEXT("Force the emulation of stereo layers instead of using native ones (if supported).\n")
	TEXT("The value of this cvar cannot be changed at runtime as it's cached during OnBeginPlay().\n")
	TEXT("Any changes made at runtime will be picked up at the next VR Preview or app startup.\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarOpenXRDoNotCopyEmulatedLayersToSpectatorScreen(
	TEXT("xr.OpenXRDoNotCopyEmulatedLayersToSpectatorScreen"),
	false,
	TEXT("If face locked stereo layers emulation is active, avoid copying the face locked stereo layers to the spectator screen.\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarOpenXRAcquireMode(
	TEXT("xr.OpenXRAcquireMode"),
	2,
	TEXT("Override the swapchain acquire mode. 1 = Acquire on any thread, 2 = Only acquire on RHI thread\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarOpenXRPreferredViewConfiguration(
	TEXT("xr.OpenXRPreferredViewConfiguration"),
	0,
	TEXT("Override the runtime's preferred view configuration if the selected configuration is available.\n")
	TEXT("1 = Mono, 2 = Stereo\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarOpenXRInvertAlpha(
	TEXT("xr.OpenXRInvertAlpha"),
	false,
	TEXT("Enables alpha inversion of the backgroud layer if the XR_EXT_composition_layer_inverted_alpha extension or XR_FB_composition_layer_alpha_blend is supported.\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarOpenXRAllowDepthLayer(
	TEXT("xr.OpenXRAllowDepthLayer"),
	true,
	TEXT("Enables the depth composition layer if the XR_KHR_composition_layer_depth extension is supported.\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarOpenXRUseWaitCountToAvoidExtraXrBeginFrameCalls(
	TEXT("xr.OpenXRUseWaitCountToAvoidExtraXrBeginFrameCalls"),
	true,
	TEXT("If true we use the WaitCount in the PipelinedFrameState to avoid extra xrBeginFrame calls.  Without this level loads can cause two additional xrBeginFrame calls.\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarOpenXRLateUpdateDeviceLocationsAfterReflections(
	TEXT("xr.OpenXRLateUpdateDeviceLocationsAfterReflections"),
	false,
	TEXT("If true, delays snapshotting device late update poses until OnBeginRendering_RenderThread, after planar reflections.\n")
	TEXT("This is necessary to get accurate late update poses for some platforms, and will reduce apparent latency, but will also cause visual lag in planar reflections.\n")
	TEXT("If you aren't using planar reflections in your project, you can safely enable this to get late update poses as late as possible.\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarOpenXRAlphaInvertPass(
	TEXT("OpenXR.AlphaInvertPass"),
	0,
	TEXT("Whether to run a render pass to un-invert the alpha value from unreal standard to the much more common standard where alpha 0 is fully transparent and alpha 1 is fully opaque.")
	TEXT("This cvar specifically enables the pass for the main XR view.  There is a more general r.ALphaInvertPass which enables it for all renders."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarOpenXRFrameSynthesis(
	TEXT("xr.OpenXRFrameSynthesis"),
	0,
	TEXT("If true and supported via XR_EXT_frame_synthesis or XR_FB_space_warp, write to and submit motion vector and motion vector depth swapchains for frame synthesis.\n")
	TEXT("Currently only supported when using the Vulkan mobile renderer, using mobile multi-view, and r.Velocity.DirectlyRenderOpenXRMotionVectors=True.\n")
	TEXT("Because normal velocity rendering is disabled when r.Velocity.DirectlyRenderOpenXRMotionVectors=True, temporal anti-aliasing and motion blur will be automatically disabled."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOpenXRProjectionLayerPriority(
	TEXT("xr.OpenXRProjectionLayerPriority"),
	0,
	TEXT("Render priority for the projection layer (main UE environment). Stereo layers with priorities greater than or equal to this value will render above the projection layer. Layers with priorities less than this value will render below the projection layer."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarOpenXRSortFaceLockedLayersAboveOtherLayers(
	TEXT("xr.OpenXRSortFaceLockedLayersAboveOtherLayers"),
	false,
	TEXT("Sort face locked stereo layers above all other layers, including the projection layer, even if the face locked layer has a lower priority. This was the standard behavior before UE 5.8."),
	ECVF_RenderThreadSafe);

namespace {
	static TSet<XrViewConfigurationType> SupportedViewConfigurations{ XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO, XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO_WITH_FOVEATED_INSET };

	/** Helper function for acquiring the appropriate FSceneViewport */
	FSceneViewport* FindSceneViewport()
	{
		if (!GIsEditor)
		{
			UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
			return GameEngine->SceneViewport.Get();
		}
	#if WITH_EDITOR
		else
		{
			UEditorEngine* EditorEngine = CastChecked<UEditorEngine>(GEngine);
			FSceneViewport* PIEViewport = (FSceneViewport*)EditorEngine->GetPIEViewport();
			if (PIEViewport != nullptr && PIEViewport->IsStereoRenderingAllowed())
			{
				// PIE is setup for stereo rendering
				return PIEViewport;
			}
			else
			{
				// Check to see if the active editor viewport is drawing in stereo mode
				// @todo vreditor: Should work with even non-active viewport!
				FSceneViewport* EditorViewport = (FSceneViewport*)EditorEngine->GetActiveViewport();
				if (EditorViewport != nullptr && EditorViewport->IsStereoRenderingAllowed())
				{
					return EditorViewport;
				}
			}
		}
	#endif
		return nullptr;
	}
}

//---------------------------------------------------
// OpenXRHMD IHeadMountedDisplay Implementation
//---------------------------------------------------

bool FOpenXRHMD::FVulkanExtensions::GetVulkanInstanceExtensionsRequired(TArray<const ANSICHAR*>& Out)
{
#ifdef XR_USE_GRAPHICS_API_VULKAN
	if (Extensions.IsEmpty())
	{
		PFN_xrGetVulkanInstanceExtensionsKHR GetVulkanInstanceExtensionsKHR;
		XR_ENSURE(xrGetInstanceProcAddr(Instance, "xrGetVulkanInstanceExtensionsKHR", (PFN_xrVoidFunction*)&GetVulkanInstanceExtensionsKHR));

		uint32 ExtensionCount = 0;
		XR_ENSURE(GetVulkanInstanceExtensionsKHR(Instance, System, 0, &ExtensionCount, nullptr));
		Extensions.SetNum(ExtensionCount);
		XR_ENSURE(GetVulkanInstanceExtensionsKHR(Instance, System, ExtensionCount, &ExtensionCount, Extensions.GetData()));
	}

	ANSICHAR* Context = nullptr;
	for (ANSICHAR* Tok = FCStringAnsi::Strtok(Extensions.GetData(), " ", &Context); Tok != nullptr; Tok = FCStringAnsi::Strtok(nullptr, " ", &Context))
	{
		Out.Add(Tok);
	}
#endif
	return true;
}

bool FOpenXRHMD::FVulkanExtensions::GetVulkanDeviceExtensionsRequired(VkPhysicalDevice_T *pPhysicalDevice, TArray<const ANSICHAR*>& Out)
{
#ifdef XR_USE_GRAPHICS_API_VULKAN
	if (DeviceExtensions.IsEmpty())
	{
		PFN_xrGetVulkanDeviceExtensionsKHR GetVulkanDeviceExtensionsKHR;
		XR_ENSURE(xrGetInstanceProcAddr(Instance, "xrGetVulkanDeviceExtensionsKHR", (PFN_xrVoidFunction*)&GetVulkanDeviceExtensionsKHR));

		uint32 ExtensionCount = 0;
		XR_ENSURE(GetVulkanDeviceExtensionsKHR(Instance, System, 0, &ExtensionCount, nullptr));
		DeviceExtensions.SetNum(ExtensionCount);
		XR_ENSURE(GetVulkanDeviceExtensionsKHR(Instance, System, ExtensionCount, &ExtensionCount, DeviceExtensions.GetData()));
	}

	ANSICHAR* Context = nullptr;
	for (ANSICHAR* Tok = FCStringAnsi::Strtok(DeviceExtensions.GetData(), " ", &Context); Tok != nullptr; Tok = FCStringAnsi::Strtok(nullptr, " ", &Context))
	{
		Out.Add(Tok);
	}
#endif
	return true;
}

void FOpenXRHMD::GetMotionControllerState(UObject* WorldContext, const EXRSpaceType XRSpaceType, const EControllerHand Hand, const EXRControllerPoseType XRControllerPoseType, FXRMotionControllerState& MotionControllerState)
{
	auto ToMotionSourceName = [](const EControllerHand Hand, const EXRControllerPoseType XRControllerPoseType)
		{
			static FLazyName LeftAim = "LeftAim";
			static FLazyName LeftGrip = "LeftGrip";
			static FLazyName LeftPalm = "LeftPalm";
			static FLazyName RightAim = "RightAim";
			static FLazyName RightGrip = "RightGrip";
			static FLazyName RightPalm = "RightPalm";
			if (Hand == EControllerHand::Left)
			{
				switch (XRControllerPoseType)
				{
				case EXRControllerPoseType::Aim:
					return LeftAim;
				case EXRControllerPoseType::Grip:
					return LeftGrip;
				case EXRControllerPoseType::Palm:
					return LeftPalm;
				default:
					check(false);
					return LeftGrip;
				}
			}
			else
			{
				switch (XRControllerPoseType)
				{
				case EXRControllerPoseType::Aim:
					return RightAim;
				case EXRControllerPoseType::Grip:
					return RightGrip;
				case EXRControllerPoseType::Palm:
					return RightPalm;
				default:
					check(false);
					return RightGrip;
				}
			}
		};

	MotionControllerState.DeviceName = NAME_None;
	MotionControllerState.ApplicationInstanceID = FApp::GetInstanceId();
	MotionControllerState.TrackingStatus = ETrackingStatus::NotTracked;
	MotionControllerState.Hand = Hand;
	MotionControllerState.XRSpaceType = XRSpaceType;
	MotionControllerState.bValid = false;

	FString InteractionProfile;
	if (GetCurrentInteractionProfile(Hand, InteractionProfile))
	{
		MotionControllerState.DeviceName = FName(InteractionProfile);
	}

	if ((Hand == EControllerHand::Left) || (Hand == EControllerHand::Right))
	{
		FName MotionControllerName("OpenXR");
		TArray<IMotionController*> MotionControllers = IModularFeatures::Get().GetModularFeatureImplementations<IMotionController>(IMotionController::GetModularFeatureName());
		IMotionController* MotionController = nullptr;
		for (auto Itr : MotionControllers)
		{
			if (Itr->GetMotionControllerDeviceTypeName() == MotionControllerName)
			{
				MotionController = Itr;
				break;
			}
		}

		if (MotionController)
		{
			{
				// Handle the pose that is actually being requested
				FName MotionSource = ToMotionSourceName(Hand, XRControllerPoseType);
				FVector Position = FVector::ZeroVector;
				FRotator Rotation = FRotator::ZeroRotator;
				FTransform TrackingToWorld = XRSpaceType == EXRSpaceType::UnrealWorldSpace ? GetTrackingToWorldTransform() : FTransform::Identity;
				const float WorldToMeters = XRSpaceType == EXRSpaceType::UnrealWorldSpace ? GetWorldToMetersScale() : 100.0f;
				bool bSuccess = MotionController->GetControllerOrientationAndPosition(0, MotionSource, Rotation, Position, WorldToMeters);
				if (bSuccess)
				{
					MotionControllerState.ControllerLocation = TrackingToWorld.TransformPosition(Position);
					MotionControllerState.ControllerRotation = TrackingToWorld.TransformRotation(FQuat(Rotation));
				}
				MotionControllerState.bValid |= bSuccess;

				MotionControllerState.TrackingStatus = MotionController->GetControllerTrackingStatus(0, MotionSource);
			}

			{
				// We always provide the grip transform in unreal space for XRVisualizationFunctionLibrary
				// THe bValid and TrackingStatus above are also valid for this pose.
				FName MotionSource = ToMotionSourceName(Hand, EXRControllerPoseType::Grip);
				FVector Position = FVector::ZeroVector;
				FRotator Rotation = FRotator::ZeroRotator;
				FTransform TrackingToWorld = GetTrackingToWorldTransform();
				bool bSuccess = MotionController->GetControllerOrientationAndPosition(0, MotionSource, Rotation, Position, GetWorldToMetersScale());
				if (bSuccess)
				{
					MotionControllerState.GripUnrealSpaceLocation = TrackingToWorld.TransformPosition(Position);
					MotionControllerState.GripUnrealSpaceRotation = TrackingToWorld.TransformRotation(FQuat(Rotation));
				}
			}
		}
	}
}

void FOpenXRHMD::GetHandTrackingState(UObject* WorldContext, const EXRSpaceType XRSpaceType, const EControllerHand Hand, FXRHandTrackingState& HandTrackingState)
{
	HandTrackingState.ApplicationInstanceID = FApp::GetInstanceId();
	HandTrackingState.TrackingStatus = ETrackingStatus::NotTracked;
	HandTrackingState.Hand = Hand;
	HandTrackingState.XRSpaceType = XRSpaceType;
	HandTrackingState.bValid = false;

	FName HandTrackerName("OpenXRHandTracking");
	TArray<IHandTracker*> HandTrackers = IModularFeatures::Get().GetModularFeatureImplementations<IHandTracker>(IHandTracker::GetModularFeatureName());
	IHandTracker* HandTracker = nullptr;
	for (auto Itr : HandTrackers)
	{
		if (Itr->GetHandTrackerDeviceTypeName() == HandTrackerName)
		{
			HandTracker = Itr;
			break;
		}
	}

	if ((Hand == EControllerHand::Left) || (Hand == EControllerHand::Right))
	{
		const float WorldToMeters = GetWorldToMetersScale();
		if (HandTracker && HandTracker->IsHandTrackingStateValid())
		{
			bool bTracked = false;
			HandTrackingState.bValid = HandTracker->GetAllKeypointStates(Hand, HandTrackingState.HandKeyLocations, HandTrackingState.HandKeyRotations, HandTrackingState.HandKeyRadii, bTracked);
			if (HandTrackingState.bValid)
			{
				HandTrackingState.TrackingStatus = bTracked ? ETrackingStatus::Tracked : ETrackingStatus::NotTracked;
			}
			check(!HandTrackingState.bValid || (HandTrackingState.HandKeyLocations.Num() == EHandKeypointCount && HandTrackingState.HandKeyRotations.Num() == EHandKeypointCount && HandTrackingState.HandKeyRadii.Num() == EHandKeypointCount));
		}
	}
}

bool FOpenXRHMD::GetCurrentInteractionProfile(const EControllerHand Hand, FString& InteractionProfile)
{
	const XrPath Path = GetUserPathForControllerHand(Hand);
	if (Path == XR_NULL_PATH)
	{
		UE_LOGF(LogHMD, Warning, "GetCurrentInteractionProfile failed because that EControllerHandValue %i does not map to a user path!", int(Hand));
		return false;
	}

	FReadScopeLock SessionLock(SessionHandleMutex);
	if (Session)
	{
		XrInteractionProfileState Profile{ XR_TYPE_INTERACTION_PROFILE_STATE };
		XrResult Result = xrGetCurrentInteractionProfile(Session, Path, &Profile);
		if (XR_SUCCEEDED(Result))
		{
			if (Profile.interactionProfile == XR_NULL_PATH)
			{
				InteractionProfile = "";
				return true;
			}
			else
			{				
				InteractionProfile = FOpenXRPath(Profile.interactionProfile);
				return true;
			}
		}
		else
		{
			FString PathStr = FOpenXRPath(Path);
			UE_LOGF(LogHMD, Warning, "GetCurrentInteractionProfile for %i (%ls) failed because xrGetCurrentInteractionProfile failed with result %ls.", int(Hand), *PathStr, OpenXRResultToString(Result));
			return false;
		}
	}
	else
	{
		UE_LOGF(LogHMD, Warning, "GetCurrentInteractionProfile for %i failed because session is null!", int(Hand));
		return false;
	}

}

float FOpenXRHMD::GetWorldToMetersScale() const
{
	return IsInActualRenderingThread() ? PipelinedFrameStateRendering.WorldToMetersScale : PipelinedFrameStateGame.WorldToMetersScale;
}

FVector2D FOpenXRHMD::GetPlayAreaBounds(EHMDTrackingOrigin::Type Origin) const
{
	XrReferenceSpaceType Space = XR_REFERENCE_SPACE_TYPE_LOCAL;
	switch (Origin)
	{
	case EHMDTrackingOrigin::View:
		Space = XR_REFERENCE_SPACE_TYPE_VIEW;
		break;
	case EHMDTrackingOrigin::Local:
		Space = XR_REFERENCE_SPACE_TYPE_LOCAL;
		break;
	case EHMDTrackingOrigin::LocalFloor:
		Space = XR_REFERENCE_SPACE_TYPE_LOCAL_FLOOR;
		break;
	case EHMDTrackingOrigin::Stage:
		Space = XR_REFERENCE_SPACE_TYPE_STAGE;
		break;
	case EHMDTrackingOrigin::CustomOpenXR:
		if (bUseCustomReferenceSpace)
		{
			Space = TrackingSpaceType;
			break;
		}
		else
		{
			UE_LOGF(LogHMD, Warning, "GetPlayAreaBounds(EHMDTrackingOrigin::CustomOpenXR), but we are not using a custom reference space now. Returning zero vector.");
			return FVector2D::ZeroVector;
		}
	default:
		check(false);

		break;
	}
	XrExtent2Df Bounds;
	const XrResult Result = xrGetReferenceSpaceBoundsRect(Session, Space, &Bounds);
	if (Result != XR_SUCCESS)
	{
		UE_LOGF(LogHMD, Warning, "GetPlayAreaBounds xrGetReferenceSpaceBoundsRect with reference space %ls failed with result %ls. Returning zero vector.", OpenXRReferenceSpaceTypeToString(Space), OpenXRResultToString(Result));
		return FVector2D::ZeroVector;
	}
	
	Swap(Bounds.width, Bounds.height); // Convert to UE coordinate system
	return ToFVector2D(Bounds, WorldToMetersScale);
}

bool FOpenXRHMD::GetPlayAreaRect(FTransform& OutTransform, FVector2D& OutRect) const
{
	// Get the origin and the extents of the play area rect.
	// The OpenXR Stage Space defines the origin of the playable rectangle.  The origin is at the floor. xrGetReferenceSpaceBoundsRect will give you the horizontal extents.

	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelinedState = LockedPipelineState.GetFrameState();

	{
		if (StageSpace == XR_NULL_HANDLE)
		{
			return false;
		}

		XrSpaceLocation NewLocation = { XR_TYPE_SPACE_LOCATION };
		const XrResult Result = xrLocateSpace(StageSpace, PipelinedState.TrackingSpace->Handle, PipelinedState.FrameState.predictedDisplayTime, &NewLocation);
		if (Result != XR_SUCCESS)
		{
			return false;
		}

		if (!(NewLocation.locationFlags & (XR_SPACE_LOCATION_POSITION_VALID_BIT | XR_SPACE_LOCATION_ORIENTATION_VALID_BIT)))
		{
			return false;
		}
		const FQuat Orientation = ToFQuat(NewLocation.pose.orientation);
		const FVector Position = ToFVector(NewLocation.pose.position, PipelinedState.WorldToMetersScale);

		FTransform TrackingToWorld = GetTrackingToWorldTransform();
		OutTransform = FTransform(Orientation, Position) * TrackingToWorld;
	}

	{
		XrExtent2Df Bounds; // width is X height is Z
		const XrResult Result = xrGetReferenceSpaceBoundsRect(Session, XR_REFERENCE_SPACE_TYPE_STAGE, &Bounds);
		if (Result != XR_SUCCESS)
		{
			return false;
		}

		OutRect = ToFVector2D(Bounds, PipelinedState.WorldToMetersScale);
	}

	return true;
}

bool FOpenXRHMD::GetTrackingOriginTransform(TEnumAsByte<EHMDTrackingOrigin::Type> Origin, FTransform& OutTransform)  const
{
	XrSpace Space = XR_NULL_HANDLE;
	switch (Origin)
	{
	case EHMDTrackingOrigin::Local:
		{
			FReadScopeLock DeviceLock(DeviceMutex);
			if (DeviceSpaces.Num())
			{
				Space = DeviceSpaces[HMDDeviceId].Space;
			}
		}
		break;
	case EHMDTrackingOrigin::LocalFloor:
		// This fallback logic probably should not exist, but changing it could break exisitng project.  
		// If we do a more comprehensive refactor of these api's we may want to eliminate this.
		Space = bLocalFloorSpaceSupported? LocalFloorSpace : LocalSpace;
		break;
	case EHMDTrackingOrigin::Stage:
		Space = StageSpace;
		break;
	case EHMDTrackingOrigin::CustomOpenXR:
		Space = CustomSpace;
		break;
	default:
		check(false);
		break;
	}

	if (Space == XR_NULL_HANDLE)
	{
		// This space is not supported.
		return false;
	}

	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelinedState = LockedPipelineState.GetFrameState();

	if (!PipelinedState.TrackingSpace.IsValid())
	{
		// Session is in a state where we can't locate.
		return false;
	}

	XrSpaceLocation NewLocation = { XR_TYPE_SPACE_LOCATION };
	const XrResult Result = xrLocateSpace(Space, PipelinedState.TrackingSpace->Handle, PipelinedState.FrameState.predictedDisplayTime, &NewLocation);
	if (Result != XR_SUCCESS)
	{
		return false;
	}
	if (!(NewLocation.locationFlags & (XR_SPACE_LOCATION_POSITION_VALID_BIT | XR_SPACE_LOCATION_ORIENTATION_VALID_BIT)))
	{
		return false;
	}
	const FQuat Orientation = ToFQuat(NewLocation.pose.orientation);
	const FVector Position = ToFVector(NewLocation.pose.position, PipelinedState.WorldToMetersScale);

	FTransform TrackingToWorld = GetTrackingToWorldTransform();
	OutTransform =  FTransform(Orientation, Position) * TrackingToWorld;

	return true;
}


FName FOpenXRHMD::GetHMDName() const
{
	return UTF8_TO_TCHAR(SystemProperties.systemName);
}

FString FOpenXRHMD::GetVersionString() const
{
	return FString::Printf(TEXT("%s (%d.%d.%d)"),
		UTF8_TO_TCHAR(InstanceProperties.runtimeName),
		XR_VERSION_MAJOR(InstanceProperties.runtimeVersion),
		XR_VERSION_MINOR(InstanceProperties.runtimeVersion),
		XR_VERSION_PATCH(InstanceProperties.runtimeVersion));
}

bool FOpenXRHMD::IsHMDConnected()
{
	return IOpenXRHMDModule::Get().GetSystemId() != XR_NULL_SYSTEM_ID;
}

bool FOpenXRHMD::IsHMDEnabled() const
{
	return true;
}

void FOpenXRHMD::EnableHMD(bool enable)
{
}

bool FOpenXRHMD::GetHMDMonitorInfo(MonitorInfo& MonitorDesc)
{
	if (!AcquireSystemIdAndProperties())
	{
		return false;
	}

	MonitorDesc.MonitorName = UTF8_TO_TCHAR(SystemProperties.systemName);
	MonitorDesc.MonitorId = 0;

	FIntPoint RTSize = GetIdealRenderTargetSize();
	MonitorDesc.DesktopX = MonitorDesc.DesktopY = 0;
	MonitorDesc.ResolutionX = MonitorDesc.WindowSizeX = RTSize.X;
	MonitorDesc.ResolutionY = MonitorDesc.WindowSizeY = RTSize.Y;
	return true;
}

void FOpenXRHMD::GetFieldOfView(float& OutHFOVInDegrees, float& OutVFOVInDegrees) const
{
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& FrameState = LockedPipelineState.GetFrameState();

	XrFovf UnifiedFov = { 0.0f };
	for (const XrView& View : FrameState.Views)
	{
		UnifiedFov.angleLeft = FMath::Min(UnifiedFov.angleLeft, View.fov.angleLeft);
		UnifiedFov.angleRight = FMath::Max(UnifiedFov.angleRight, View.fov.angleRight);
		UnifiedFov.angleUp = FMath::Max(UnifiedFov.angleUp, View.fov.angleUp);
		UnifiedFov.angleDown = FMath::Min(UnifiedFov.angleDown, View.fov.angleDown);
	}
	OutHFOVInDegrees = FMath::RadiansToDegrees(UnifiedFov.angleRight - UnifiedFov.angleLeft);
	OutVFOVInDegrees = FMath::RadiansToDegrees(UnifiedFov.angleUp - UnifiedFov.angleDown);
}

void FOpenXRHMD::GetFullFOVInformation(TArray<XrFovf>& FovInfos) const
{
	FovInfos.Empty();

	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& FrameState = LockedPipelineState.GetFrameState();
	for (const XrView& View : FrameState.Views)
	{
		FovInfos.Add(View.fov);
	}
}

bool FOpenXRHMD::EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type)
{
	if (Type == EXRTrackedDeviceType::Any || Type == EXRTrackedDeviceType::HeadMountedDisplay)
	{
		OutDevices.Add(IXRTrackingSystem::HMDDeviceId);
	}
	if (Type == EXRTrackedDeviceType::Any || Type == EXRTrackedDeviceType::Controller)
	{
		FReadScopeLock DeviceLock(DeviceMutex);

		// Skip the HMD, we already added it to the list
		for (int32 i = 1; i < DeviceSpaces.Num(); i++)
		{
			OutDevices.Add(i);
		}
	}
	return OutDevices.Num() > 0;
}

void FOpenXRHMD::SetInterpupillaryDistance(float NewInterpupillaryDistance)
{
}

float FOpenXRHMD::GetInterpupillaryDistance() const
{
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& FrameState = LockedPipelineState.GetFrameState();
	if (FrameState.Views.Num() < 2)
	{
		return 0.064f;
	}

	FVector leftPos = ToFVector(FrameState.Views[0].pose.position);
	FVector rightPos = ToFVector(FrameState.Views[1].pose.position);
	return FVector::Dist(leftPos, rightPos);
}	

bool FOpenXRHMD::IsTracking(int32 DeviceId)
{
	// This function is called from both the game and rendering thread and each thread maintains separate pose
	// snapshots to prevent inconsistent poses (tearing) on the same frame.
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = (const_cast<const FOpenXRHMD*>(this))->GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();

	if (!PipelineState.DeviceLocations.IsValidIndex(DeviceId))
	{
		return false;
	}

	const XrSpaceLocation& Location = PipelineState.DeviceLocations[DeviceId];
	return Location.locationFlags & XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT &&
		Location.locationFlags & XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
}

bool FOpenXRHMD::HasValidTrackingPosition()
{
	FQuat Orientation;
	FVector Position;
	return GetCurrentPose(HMDDeviceId, Orientation, Position);
}

bool FOpenXRHMD::GetIsTracked(int32 DeviceId)
{
	return IsTracking(DeviceId);
}

bool FOpenXRHMD::GetCurrentPose(int32 DeviceId, FQuat& CurrentOrientation, FVector& CurrentPosition)
{
	CurrentOrientation = FQuat::Identity;
	CurrentPosition = FVector::ZeroVector;

	// This function is called from both the game and rendering thread and each thread maintains separate pose
	// snapshots to prevent inconsistent poses (tearing) on the same frame.
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = (const_cast<const FOpenXRHMD*>(this))->GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();

	if (!PipelineState.DeviceLocations.IsValidIndex(DeviceId))
	{
		return false;
	}

	const XrSpaceLocation& Location = PipelineState.DeviceLocations[DeviceId];
	if (Location.locationFlags & XR_SPACE_LOCATION_ORIENTATION_VALID_BIT)
	{
		CurrentOrientation = ToFQuat(Location.pose.orientation);
	}
	if (Location.locationFlags & XR_SPACE_LOCATION_POSITION_VALID_BIT)
	{
		CurrentPosition = ToFVector(Location.pose.position, GetWorldToMetersScale());
	}
	return (Location.locationFlags & XR_SPACE_LOCATION_ORIENTATION_VALID_BIT) && (Location.locationFlags & XR_SPACE_LOCATION_POSITION_VALID_BIT);
}

bool FOpenXRHMD::GetPoseForTime(int32 DeviceId, FTimespan Timespan, bool& OutTimeWasUsed, FQuat& Orientation, FVector& Position, bool& bProvidedLinearVelocity, FVector& LinearVelocity, bool& bProvidedAngularVelocity, FVector& AngularVelocityAsAxisAndLength, bool& bProvidedLinearAcceleration, FVector& LinearAcceleration, float InWorldToMetersScale)
{
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = (const_cast<const FOpenXRHMD*>(this))->GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();

	FReadScopeLock DeviceLock(DeviceMutex);
	if (!DeviceSpaces.IsValidIndex(DeviceId))
	{
		return false;
	}

	XrTime TargetTime = ToXrTime(Timespan);

	// If TargetTime is zero just get the latest data (rather than the oldest).
	if (TargetTime == 0)
	{
		OutTimeWasUsed = false;
		TargetTime = GetDisplayTime();

		
		if (TargetTime == 0)
		{
			// We might still get an out-of-sync query after the session has ended.
			// We could return the last known location via PipelineState.DeviceLocations
			// but UpdateDeviceLocations doesn't do that right now. We'll just fail for now.
			return false;
		}
	}
	else
	{
		OutTimeWasUsed = true;
	}

	const FDeviceSpace& DeviceSpace = DeviceSpaces[DeviceId];

	XrSpaceAccelerationEPIC DeviceAcceleration{ (XrStructureType)XR_TYPE_SPACE_ACCELERATION_EPIC };
	void* DeviceAccelerationPtr = bSpaceAccelerationSupported ? &DeviceAcceleration : nullptr;
	XrSpaceVelocity DeviceVelocity { XR_TYPE_SPACE_VELOCITY, DeviceAccelerationPtr };
	XrSpaceLocation DeviceLocation { XR_TYPE_SPACE_LOCATION, &DeviceVelocity };

	XR_ENSURE(xrLocateSpace(DeviceSpace.Space, PipelineState.TrackingSpace->Handle, TargetTime, &DeviceLocation));

	bool ReturnValue = false;

	if (DeviceLocation.locationFlags & XR_SPACE_LOCATION_ORIENTATION_VALID_BIT &&
		DeviceLocation.locationFlags & XR_SPACE_LOCATION_POSITION_VALID_BIT)
	{
		Orientation = ToFQuat(DeviceLocation.pose.orientation);
		Position = ToFVector(DeviceLocation.pose.position, InWorldToMetersScale);

		if (DeviceVelocity.velocityFlags & XR_SPACE_VELOCITY_LINEAR_VALID_BIT)
		{
			bProvidedLinearVelocity = true;
			LinearVelocity = ToFVector(DeviceVelocity.linearVelocity, InWorldToMetersScale);
		}
		if (DeviceVelocity.velocityFlags & XR_SPACE_VELOCITY_ANGULAR_VALID_BIT)
		{
			bProvidedAngularVelocity = true;
			// Convert to unreal coordinate system & LeftHanded rotation.  
			// We cannot use quaternion because it cannot represent rotations beyond 180/sec.  
			// We don't want to use FRotator because it is hard to transform with the TrackingToWorldTransform.
			// So this is an axis vector who's length is the angle in radians.
			AngularVelocityAsAxisAndLength = -ToFVector(DeviceVelocity.angularVelocity); 
		}

		if (DeviceAcceleration.accelerationFlags & XR_SPACE_ACCELERATION_LINEAR_VALID_BIT_EPIC)
		{
			bProvidedLinearAcceleration = true;
			LinearAcceleration = ToFVector(DeviceAcceleration.linearAcceleration, InWorldToMetersScale);
		}

		ReturnValue = true;
	}

	return ReturnValue;
}

bool FOpenXRHMD::IsChromaAbCorrectionEnabled() const
{
	return false;
}

void FOpenXRHMD::ResetOrientationAndPosition(float Yaw)
{
	Recenter(EOrientPositionSelector::OrientationAndPosition, Yaw);
}

void FOpenXRHMD::ResetOrientation(float Yaw)
{
	Recenter(EOrientPositionSelector::Orientation, Yaw);
}

void FOpenXRHMD::ResetPosition()
{

	Recenter(EOrientPositionSelector::Position);
}

void FOpenXRHMD::Recenter(EOrientPositionSelector::Type Selector, float Yaw)
{
	const XrTime TargetTime = GetDisplayTime();
	if (TargetTime == 0)
	{
		UE_LOGF(LogHMD, Warning, "Could not retrieve a valid head pose for recentering.");
		return;
	}

	XrSpace DeviceSpace = XR_NULL_HANDLE;
	{
		FReadScopeLock DeviceLock(DeviceMutex);
		const FDeviceSpace& DeviceSpaceStruct = DeviceSpaces[HMDDeviceId];
		DeviceSpace = DeviceSpaceStruct.Space;
	}
	XrSpaceLocation DeviceLocation = { XR_TYPE_SPACE_LOCATION, nullptr };

	XrSpace BaseSpace = XR_NULL_HANDLE;
	if (TrackingSpaceType == XR_REFERENCE_SPACE_TYPE_LOCAL_FLOOR)
	{
		BaseSpace = LocalFloorSpace;
	}
	else if (TrackingSpaceType == XR_REFERENCE_SPACE_TYPE_STAGE)
	{
		BaseSpace = StageSpace;
	}
	else
	{
		BaseSpace = LocalSpace;
	}
	if (bUseCustomReferenceSpace)
	{
		BaseSpace = CustomSpace;
	}
	XR_ENSURE(xrLocateSpace(DeviceSpace, BaseSpace, TargetTime, &DeviceLocation));

	const FQuat CurrentOrientation = ToFQuat(DeviceLocation.pose.orientation);
	const FVector CurrentPosition = ToFVector(DeviceLocation.pose.position, GetWorldToMetersScale());

	if (Selector == EOrientPositionSelector::Position ||
		Selector == EOrientPositionSelector::OrientationAndPosition)
	{
		FVector NewPosition;
		NewPosition.X = CurrentPosition.X;
		NewPosition.Y = CurrentPosition.Y;
		if (TrackingSpaceType == XR_REFERENCE_SPACE_TYPE_LOCAL)
		{
			NewPosition.Z = CurrentPosition.Z;
		}
		else
		{
			NewPosition.Z = 0.0f;
		}
		SetBasePosition(NewPosition);
	}

	if (Selector == EOrientPositionSelector::Orientation ||
		Selector == EOrientPositionSelector::OrientationAndPosition)
	{
		FRotator NewOrientation(0.0f, CurrentOrientation.Rotator().Yaw - Yaw, 0.0f);
		SetBaseOrientation(NewOrientation.Quaternion());
	}

	bTrackingSpaceInvalid = true;
	OnTrackingOriginChanged();
}

void FOpenXRHMD::SetBaseRotation(const FRotator& InBaseRotation)
{
	SetBaseOrientation(InBaseRotation.Quaternion());
}

FRotator FOpenXRHMD::GetBaseRotation() const
{
	return BaseOrientation.Rotator();
}

void FOpenXRHMD::SetBaseOrientation(const FQuat& InBaseOrientation)
{
	BaseOrientation = InBaseOrientation;
	bTrackingSpaceInvalid = true;
}

FQuat FOpenXRHMD::GetBaseOrientation() const
{
	return BaseOrientation;
}

void FOpenXRHMD::SetBasePosition(const FVector& InBasePosition)
{
	BasePosition = InBasePosition;
	bTrackingSpaceInvalid = true;
}

FVector FOpenXRHMD::GetBasePosition() const
{
	return BasePosition;
}

void FOpenXRHMD::SetTrackingOrigin(EHMDTrackingOrigin::Type NewOrigin)
{
	if (NewOrigin == EHMDTrackingOrigin::View)
	{
		UE_LOGF(LogHMD, Warning, "SetTrackingOrigin(EHMDTrackingOrigin::View) called, which is invalid (We allow getting the view transform as a tracking space, but we do not allow setting the tracking space origin to the View).  We are setting the tracking space to Local, to maintain legacy behavior, however ideally the blueprint calling this would be fixed to use Local space.");
		TrackingSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL;  // Local space is always supported
	}

	if (NewOrigin == EHMDTrackingOrigin::CustomOpenXR)
	{
		if (!bUseCustomReferenceSpace)
		{
			UE_LOGF(LogHMD, Warning, "SetTrackingOrigin(EHMDTrackingOrigin::CustomOpenXR) called when bUseCustomReferenceSpace is false.  This call is being ignored.  Reference space will remain %ls.", OpenXRReferenceSpaceTypeToString(TrackingSpaceType));
			return;
		}
		// The case, where we set to custom and custom is supported doesn't need to do anything.
		// It isn't really useful to do this, but it is easy to imagine that allowing it to happen might make implementing a project that supports multiple types of reference spaces easier.
		return;
	}
	
	if (bUseCustomReferenceSpace)
	{
		UE_LOGF(LogHMD, Warning, "SetTrackingOrigin(%i) called when bUseCustomReferenceSpace is true.  This call is being ignored.  Reference space will remain custom %ls.", NewOrigin, OpenXRReferenceSpaceTypeToString(TrackingSpaceType));
		return;
	}

	if (NewOrigin == EHMDTrackingOrigin::LocalFloor && (LocalFloorSpace != nullptr))
	{
		TrackingSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL_FLOOR;
	}
	else if (NewOrigin == EHMDTrackingOrigin::Local)
	{
		TrackingSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL;  // Local space is always supported, but we only prefer it if it was requested.
	}
	else if (StageSpace != nullptr) // Either stage is requested, or floor was requested but floor is not supported (stage meets the requirements for floor, and more).
	{
		TrackingSpaceType = XR_REFERENCE_SPACE_TYPE_STAGE;
	}
	else
	{
		TrackingSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL;
	}

	// Force the tracking space to refresh next frame
	bTrackingSpaceInvalid = true;
}

EHMDTrackingOrigin::Type FOpenXRHMD::GetTrackingOrigin() const
{
	switch (TrackingSpaceType)
	{
	case XR_REFERENCE_SPACE_TYPE_STAGE:
		return EHMDTrackingOrigin::Stage;
	case XR_REFERENCE_SPACE_TYPE_LOCAL_FLOOR:
		return EHMDTrackingOrigin::LocalFloor;
	case XR_REFERENCE_SPACE_TYPE_LOCAL:
		return EHMDTrackingOrigin::Local;
	case XR_REFERENCE_SPACE_TYPE_VIEW:
		check(false); // Note: we do not expect this to actually happen because view cannot be the tracking origin.
		return EHMDTrackingOrigin::View;
	default:
		if (bUseCustomReferenceSpace)
		{
			// The custom reference space covers multiple potential extension tracking origins
			return EHMDTrackingOrigin::CustomOpenXR;
		}
		else
		{
			UE_LOGF(LogHMD, Warning, "GetTrackingOrigin() called when unexpected tracking space %ls is in use.  Returning EHMDTrackingOrigin::Local because it gives the fewest guarantees, but this value is not correct!  Perhaps this function needs to support more TrackingSpaceTypes?", OpenXRReferenceSpaceTypeToString(TrackingSpaceType));
			check(false);
			return EHMDTrackingOrigin::Local;
		}
	}
}

bool FOpenXRHMD::IsStereoEnabled() const
{
	return bStereoEnabled;
}

bool FOpenXRHMD::EnableStereo(bool stereo)
{
	if (stereo == bStereoEnabled)
	{
		return true;
	}

	if (bIsTrackingOnlySession)
	{
		return false;
	}

	bStereoEnabled = stereo;
	if (stereo)
	{
		GEngine->bForceDisableFrameRateSmoothing = true;
		if (OnStereoStartup())
		{
			if (!GIsEditor)
			{
				GEngine->SetMaxFPS(0);
			}

			// Note: This StartSession may not work, but if not we should receive a SESSION_STATE_READY and try again or a LOSS_PENDING and session destruction
			StartSession();

			FApp::SetUseVRFocus(true);
			FApp::SetHasVRFocus(true);

#if WITH_EDITOR
			if (GIsEditor)
			{
				if (FSceneViewport* SceneVP = FindSceneViewport())
				{
					TSharedPtr<SWindow> Window = SceneVP->FindWindow();
					if (Window.IsValid())
					{
						uint32 SizeX = 0;
						uint32 SizeY = 0;
						CalculateRenderTargetSize(*SceneVP, SizeX, SizeY);

						// Window continues to be processed when PIE spectator window is minimized
						Window->SetIndependentViewportSize(FVector2D(SizeX, SizeY));
					}
				}
			}

#endif // WITH_EDITOR

			return true;
		}
		bStereoEnabled = false;
		return false;
	}
	else
	{
		GEngine->bForceDisableFrameRateSmoothing = false;

		FApp::SetUseVRFocus(false);
		FApp::SetHasVRFocus(false);

#if WITH_EDITOR
		if (GIsEditor)
		{
			if (FSceneViewport* SceneVP = FindSceneViewport())
			{
				TSharedPtr<SWindow> Window = SceneVP->FindWindow();
				if (Window.IsValid())
				{
					Window->SetViewportSizeDrivenByWindow(true);
				}
			}
		}
#endif // WITH_EDITOR

		return OnStereoTeardown();
	}
}

FIntPoint GeneratePixelDensitySize(const XrViewConfigurationView& Config, const float PixelDensity)
{
	FIntPoint DensityAdjustedSize =
	{
		FMath::CeilToInt(Config.recommendedImageRectWidth * PixelDensity),
		FMath::CeilToInt(Config.recommendedImageRectHeight * PixelDensity)
	};

	// We quantize in order to be consistent with the rest of the engine in creating our buffers.
	// Interestingly, we need to be a bit careful with this quantization during target alloc because
	// some runtime compositors want/expect targets that match the recommended size. Some runtimes
	// might blit from a 'larger' size to the recommended size. This could happen with quantization
	// factors that don't align with the recommended size.
	QuantizeSceneBufferSize(DensityAdjustedSize, DensityAdjustedSize);

	return DensityAdjustedSize;
}

void FOpenXRHMD::AdjustViewRect(int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const
{
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();
	const XrViewConfigurationView& Config = PipelineState.ViewConfigs[ViewIndex];
	FIntPoint ViewRectMin(EForceInit::ForceInitToZero);

	// If Mobile Multi-View is active the first two views will share the same position
	// Thus the start index should be the second view if enabled
	for (int32 i = bIsMobileMultiViewEnabled ? 1 : 0; i < ViewIndex; ++i)
	{
		ViewRectMin.X += FMath::CeilToInt(PipelineState.ViewConfigs[i].recommendedImageRectWidth * PipelineState.PixelDensity);
		QuantizeSceneBufferSize(ViewRectMin, ViewRectMin);
	}

	X = ViewRectMin.X;
	Y = ViewRectMin.Y;

	const FIntPoint DensityAdjustedSize = GeneratePixelDensitySize(Config, PipelineState.PixelDensity);

	SizeX = DensityAdjustedSize.X;
	SizeY = DensityAdjustedSize.Y;

}

void FOpenXRHMD::CalculateRenderTargetSize(const FViewport& Viewport, uint32& InOutSizeX, uint32& InOutSizeY)
{
	check(IsInGameThread() || IsInRenderingThread());

	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = (const_cast<const FOpenXRHMD*>(this))->GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();
	const float PixelDensity = PipelineState.PixelDensity;

	// TODO: Could we just call AdjustViewRect per view, or even for _only_ the last view?
	FIntPoint Size(EForceInit::ForceInitToZero);
	for (int32 ViewIndex = 0; ViewIndex < PipelineState.ViewConfigs.Num(); ViewIndex++)
	{
		const XrViewConfigurationView& Config = PipelineState.ViewConfigs[ViewIndex];

		// If Mobile Multi-View is active the first two views will share the same position
		// TODO: This is weird logic that we should re-investigate. It makes sense for AdjustViewRect, but not
		// for the 'size' of an RT.
		const bool bMMVView = bIsMobileMultiViewEnabled && ViewIndex < 2;

		const FIntPoint DensityAdjustedSize = GeneratePixelDensitySize(Config, PipelineState.PixelDensity);
		Size.X = bMMVView ? FMath::Max(Size.X, DensityAdjustedSize.X) : Size.X + DensityAdjustedSize.X;
		Size.Y = FMath::Max(Size.Y, DensityAdjustedSize.Y);
	}

	InOutSizeX = Size.X;
	InOutSizeY = Size.Y;

	check(InOutSizeX != 0 && InOutSizeY != 0);
}

void FOpenXRHMD::SetFinalViewRect(FRHICommandListImmediate& RHICmdList, const int32 ViewIndex, const FIntRect& FinalViewRect)
{
	check(IsInRenderingThread());

	if (ViewIndex == INDEX_NONE || !PipelinedLayerStateRendering.ColorImages.IsValidIndex(ViewIndex))
	{
		return;
	}

	XrSwapchainSubImage& ColorImage = PipelinedLayerStateRendering.ColorImages[ViewIndex];
	ColorImage.imageArrayIndex = bIsMobileMultiViewEnabled && ViewIndex < 2 ? ViewIndex : 0;
	ColorImage.imageRect = {
		{ FinalViewRect.Min.X, FinalViewRect.Min.Y },
		{ FinalViewRect.Width(), FinalViewRect.Height() }
	};

	XrSwapchainSubImage& DepthImage = PipelinedLayerStateRendering.DepthImages[ViewIndex];
	DepthImage.imageArrayIndex = ColorImage.imageArrayIndex;
	DepthImage.imageRect = ColorImage.imageRect;

	XrSwapchainSubImage& EmulationImage = PipelinedLayerStateRendering.EmulatedLayerState.EmulationImages[ViewIndex];
	EmulationImage.imageArrayIndex = ColorImage.imageArrayIndex;
	EmulationImage.imageRect = ColorImage.imageRect;

	FIntPoint MotionVectorSize;
	if (GetRecommendedMotionVectorTextureSize(MotionVectorSize))
	{
		const FIntRect MotionVectorRect(FIntPoint(0, 0), MotionVectorSize);

		XrSwapchainSubImage& MotionVectorImage = PipelinedLayerStateRendering.MotionVectorImages[ViewIndex];
		MotionVectorImage.imageArrayIndex = ColorImage.imageArrayIndex;
		MotionVectorImage.imageRect = {
			{ MotionVectorRect.Min.X, MotionVectorRect.Min.Y },
			{ MotionVectorRect.Width(), MotionVectorRect.Height() }
		};

		XrSwapchainSubImage& MotionVectorDepthImage = PipelinedLayerStateRendering.MotionVectorDepthImages[ViewIndex];
		MotionVectorDepthImage.imageArrayIndex = ColorImage.imageArrayIndex;
		MotionVectorDepthImage.imageRect = MotionVectorImage.imageRect;
	}
}

EStereoscopicPass FOpenXRHMD::GetViewPassForIndex(bool bStereoRequested, int32 ViewIndex) const
{
	if (!bStereoRequested)
		return EStereoscopicPass::eSSP_FULL;

	if (SelectedViewConfigurationType == XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO_WITH_FOVEATED_INSET)
	{
		return ViewIndex % 2 == 0 ? EStereoscopicPass::eSSP_PRIMARY : EStereoscopicPass::eSSP_SECONDARY;
	}
	return ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE ? EStereoscopicPass::eSSP_PRIMARY : EStereoscopicPass::eSSP_SECONDARY;
}

uint32 FOpenXRHMD::GetLODViewIndex() const
{
	if (SelectedViewConfigurationType == XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO_WITH_FOVEATED_INSET)
	{
		return EStereoscopicEye::eSSE_LEFT_EYE_SIDE;
	}
	return IStereoRendering::GetLODViewIndex();
}

int32 FOpenXRHMD::GetDesiredNumberOfViews(bool bStereoRequested) const
{
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = (const_cast<const FOpenXRHMD*>(this))->GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& FrameState = LockedPipelineState.GetFrameState();

	// FIXME: Monoscopic actually needs 2 views for quad vr
	return bStereoRequested ? FrameState.ViewConfigs.Num() : 1;
}

bool FOpenXRHMD::GetRelativeEyePose(int32 InDeviceId, int32 InViewIndex, FQuat& OutOrientation, FVector& OutPosition)
{
	if (InDeviceId != IXRTrackingSystem::HMDDeviceId)
	{
		return false;
	}

	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = (const_cast<const FOpenXRHMD*>(this))->GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& FrameState = LockedPipelineState.GetFrameState();

	if (FrameState.ViewState.viewStateFlags & XR_VIEW_STATE_ORIENTATION_VALID_BIT &&
		FrameState.ViewState.viewStateFlags & XR_VIEW_STATE_POSITION_VALID_BIT &&
		FrameState.Views.IsValidIndex(InViewIndex))
	{
		OutOrientation = ToFQuat(FrameState.Views[InViewIndex].pose.orientation);
		OutPosition = ToFVector(FrameState.Views[InViewIndex].pose.position, GetWorldToMetersScale());
		return true;
	}

	return false;
}

FMatrix FOpenXRHMD::GetStereoProjectionMatrix(const int32 ViewIndex) const
{
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& FrameState = LockedPipelineState.GetFrameState();

	XrFovf Fov = {};
	if (ViewIndex == eSSE_MONOSCOPIC)
	{
		// The monoscopic projection matrix uses the combined field-of-view of both eyes
		for (int32 Index = 0; Index < FrameState.Views.Num(); Index++)
		{
			const XrFovf& ViewFov = FrameState.Views[Index].fov;
			Fov.angleUp = FMath::Max(Fov.angleUp, ViewFov.angleUp);
			Fov.angleDown = FMath::Min(Fov.angleDown, ViewFov.angleDown);
			Fov.angleLeft = FMath::Min(Fov.angleLeft, ViewFov.angleLeft);
			Fov.angleRight = FMath::Max(Fov.angleRight, ViewFov.angleRight);
		}
	}
	else
	{
		Fov = (ViewIndex < FrameState.Views.Num()) ? FrameState.Views[ViewIndex].fov
			: XrFovf{ -PI / 4.0f, PI / 4.0f, PI / 4.0f, -PI / 4.0f };
	}

	Fov.angleUp = tan(Fov.angleUp);
	Fov.angleDown = tan(Fov.angleDown);
	Fov.angleLeft = tan(Fov.angleLeft);
	Fov.angleRight = tan(Fov.angleRight);

	float ZNear = GNearClippingPlane_RenderThread;
	float SumRL = (Fov.angleRight + Fov.angleLeft);
	float SumTB = (Fov.angleUp + Fov.angleDown);
	float InvRL = (1.0f / (Fov.angleRight - Fov.angleLeft));
	float InvTB = (1.0f / (Fov.angleUp - Fov.angleDown));

	FMatrix Mat = FMatrix(
		FPlane((2.0f * InvRL), 0.0f, 0.0f, 0.0f),
		FPlane(0.0f, (2.0f * InvTB), 0.0f, 0.0f),
		FPlane((SumRL * -InvRL), (SumTB * -InvTB), 0.0f, 1.0f),
		FPlane(0.0f, 0.0f, ZNear, 0.0f)
	);

	return Mat;
}

void FOpenXRHMD::GetEyeRenderParams_RenderThread(const FHeadMountedDisplayPassContext& Context, FVector2D& EyeToSrcUVScaleValue, FVector2D& EyeToSrcUVOffsetValue) const
{
	EyeToSrcUVOffsetValue = FVector2D::ZeroVector;
	EyeToSrcUVScaleValue = FVector2D(1.0f, 1.0f);
}


void FOpenXRHMD::SetupViewFamily(FSceneViewFamily& InViewFamily)
{
	InViewFamily.EngineShowFlags.MotionBlur = 0;
	InViewFamily.EngineShowFlags.HMDDistortion = false;
	InViewFamily.EngineShowFlags.StereoRendering = IsStereoEnabled();

	// For now we are enabling the invert alpha pass with a cvar.
	// However it seems likely that we might know we are providing alpha to openxr, and that
	// the current runtime does not support unreal's inverted alpha and set this based on that.
	if (InViewFamily.Scene)
	{
		EShaderPlatform Platform = InViewFamily.Scene->GetShaderPlatform();
		static FShaderPlatformCachedIniValue<bool> AlphaInvertPassIniValue(TEXT("OpenXR.AlphaInvertPass"));
		InViewFamily.EngineShowFlags.AlphaInvert = AlphaInvertPassIniValue.Get(Platform);
	}
	else
	{
		static auto CVarAlphaInvertPassLocal = IConsoleManager::Get().FindConsoleVariable(TEXT("OpenXR.AlphaInvertPass"));
		InViewFamily.EngineShowFlags.AlphaInvert = (CVarAlphaInvertPassLocal != nullptr) && CVarAlphaInvertPassLocal->GetBool();
	}

	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = (const_cast<const FOpenXRHMD*>(this))->GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& FrameState = LockedPipelineState.GetFrameState();
	if (FrameState.Views.Num() > 2)
	{
		InViewFamily.EngineShowFlags.Vignette = 0;
		InViewFamily.EngineShowFlags.Bloom = 0;
	}

	if (InViewFamily.Scene)
	{
		EShaderPlatform ShaderPlatform = InViewFamily.Scene->GetShaderPlatform();
		if (PlatformSupportsOpenXRMotionVectors(ShaderPlatform) && bIsMobileMultiViewEnabled)
		{
			static FShaderPlatformCachedIniValue<bool> FrameSynthesisIniValue(TEXT("xr.OpenXRFrameSynthesis"));
			const bool bFrameSynthesisCVarEnabled = FrameSynthesisIniValue.Get(ShaderPlatform);
			const bool bFrameSynthesisExtensionSupported =
				IsExtensionEnabled(XR_EXT_FRAME_SYNTHESIS_EXTENSION_NAME) ||
				IsExtensionEnabled(XR_FB_SPACE_WARP_EXTENSION_NAME);

			if (bFrameSynthesisCVarEnabled && bFrameSynthesisExtensionSupported)
			{
				InViewFamily.EngineShowFlags.StereoMotionVectors = true;
			}
		}
	}
}

void FOpenXRHMD::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
}

void FOpenXRHMD::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	uint32 ViewConfigCount = 0;
	XR_ENSURE(xrEnumerateViewConfigurationViews(Instance, System, SelectedViewConfigurationType, 0, &ViewConfigCount, nullptr));

	PipelinedLayerStateRendering.ProjectionLayers.SetNum(ViewConfigCount);
	PipelinedLayerStateRendering.DepthLayers.SetNum(ViewConfigCount);
	PipelinedLayerStateRendering.EmulatedLayerState.CompositedProjectionLayers.SetNum(ViewConfigCount);

	PipelinedLayerStateRendering.FrameSynthesisLayers.SetNum(ViewConfigCount);
	PipelinedLayerStateRendering.SpaceWarpLayers.SetNum(ViewConfigCount);

	PipelinedLayerStateRendering.ColorImages.SetNum(PipelinedFrameStateRendering.ViewConfigs.Num());
	PipelinedLayerStateRendering.DepthImages.SetNum(PipelinedFrameStateRendering.ViewConfigs.Num());
	PipelinedLayerStateRendering.EmulatedLayerState.EmulationImages.SetNum(PipelinedFrameStateRendering.ViewConfigs.Num());

	PipelinedLayerStateRendering.MotionVectorImages.SetNum(PipelinedFrameStateRendering.ViewConfigs.Num());
	PipelinedLayerStateRendering.MotionVectorDepthImages.SetNum(PipelinedFrameStateRendering.ViewConfigs.Num());

	if (bCompositionLayerColorScaleBiasSupported)
	{
		PipelinedLayerStateRendering.LayerColorScaleAndBias = { LayerColorScale, LayerColorBias };
	}

	if (SpectatorScreenController)
	{
		SpectatorScreenController->BeginRenderViewFamily(InViewFamily);
	}
}

void FOpenXRHMD::PostRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView)
{
	DrawEmulatedLayers_RenderThread(GraphBuilder, InView);
}

void FOpenXRHMD::PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	check(IsInRenderingThread());

	const float NearZ = GNearClippingPlane_RenderThread / GetWorldToMetersScale();

	for (int32 ViewIndex = 0; ViewIndex < PipelinedLayerStateRendering.ColorImages.Num(); ViewIndex++)
	{
		if (!PipelinedLayerStateRendering.ColorImages.IsValidIndex(ViewIndex))
		{
			continue;
		}

		// Update SubImages with latest swapchain
		XrSwapchainSubImage& ColorImage = PipelinedLayerStateRendering.ColorImages[ViewIndex];
		XrSwapchainSubImage& DepthImage = PipelinedLayerStateRendering.DepthImages[ViewIndex];
		XrSwapchainSubImage& EmulationImage = PipelinedLayerStateRendering.EmulatedLayerState.EmulationImages[ViewIndex];
		XrSwapchainSubImage& MotionVectorImage = PipelinedLayerStateRendering.MotionVectorImages[ViewIndex];
		XrSwapchainSubImage& MotionVectorDepthImage = PipelinedLayerStateRendering.MotionVectorDepthImages[ViewIndex];

		ColorImage.swapchain = PipelinedLayerStateRendering.ColorSwapchain.IsValid() ? static_cast<FOpenXRSwapchain*>(PipelinedLayerStateRendering.ColorSwapchain.Get())->GetHandle() : XR_NULL_HANDLE;
		if (EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitDepthLayer))
		{
			DepthImage.swapchain = PipelinedLayerStateRendering.DepthSwapchain.IsValid() ? static_cast<FOpenXRSwapchain*>(PipelinedLayerStateRendering.DepthSwapchain.Get())->GetHandle() : XR_NULL_HANDLE;
		}
		if (EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitEmulatedFaceLockedLayer))
		{
			EmulationImage.swapchain = PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain.IsValid() ? static_cast<FOpenXRSwapchain*>(PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain.Get())->GetHandle() : XR_NULL_HANDLE;
		}
		if (EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitMotionVectorLayer))
		{
			check(PipelinedLayerStateRendering.MotionVectorSwapchain && PipelinedLayerStateRendering.MotionVectorDepthSwapchain);

			MotionVectorImage.swapchain = PipelinedLayerStateRendering.MotionVectorSwapchain.IsValid() ? static_cast<FOpenXRSwapchain*>(PipelinedLayerStateRendering.MotionVectorSwapchain.Get())->GetHandle() : XR_NULL_HANDLE;
			MotionVectorDepthImage.swapchain = PipelinedLayerStateRendering.MotionVectorDepthSwapchain.IsValid() ? static_cast<FOpenXRSwapchain*>(PipelinedLayerStateRendering.MotionVectorDepthSwapchain.Get())->GetHandle() : XR_NULL_HANDLE;
		}

		XrCompositionLayerProjectionView& Projection = PipelinedLayerStateRendering.ProjectionLayers[ViewIndex];

		Projection.type = XR_TYPE_COMPOSITION_LAYER_PROJECTION_VIEW;
		Projection.next = nullptr;
		Projection.subImage = ColorImage;

		if (EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitDepthLayer))
		{
			XrCompositionLayerDepthInfoKHR& DepthLayer = PipelinedLayerStateRendering.DepthLayers[ViewIndex];

			DepthLayer.type = XR_TYPE_COMPOSITION_LAYER_DEPTH_INFO_KHR;
			DepthLayer.next = nullptr;
			DepthLayer.subImage = DepthImage;
			DepthLayer.minDepth = 0.0f;
			DepthLayer.maxDepth = 1.0f;
			DepthLayer.nearZ = FLT_MAX;
			DepthLayer.farZ = NearZ;

			for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
			{
				DepthLayer.next = Module->OnBeginDepthInfo(Session, 0, ViewIndex, DepthLayer.next);
			}

			Projection.next = &DepthLayer;
		}
		if (EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitEmulatedFaceLockedLayer))
		{
			XrCompositionLayerProjectionView& CompositedProjection = PipelinedLayerStateRendering.EmulatedLayerState.CompositedProjectionLayers[ViewIndex];

			CompositedProjection.type = XR_TYPE_COMPOSITION_LAYER_PROJECTION_VIEW;
			CompositedProjection.next = nullptr;
			CompositedProjection.subImage = EmulationImage;
		}
		if (EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitMotionVectorLayer))
		{
			if (IsExtensionEnabled(XR_EXT_FRAME_SYNTHESIS_EXTENSION_NAME))
			{
				XrFrameSynthesisInfoEXT& FrameSynthesisLayerInfo = PipelinedLayerStateRendering.FrameSynthesisLayers[ViewIndex];

				FTransform TrackingToWorld = GetTrackingToWorldTransform();
				FTransform TrackingSpaceDeltaPose = TrackingToWorld * FrameSynthesisLastTrackingToWorld.Inverse();
				FrameSynthesisLastTrackingToWorld = TrackingToWorld;
				FTransform BaseTransform = FTransform(GetBaseOrientation(), GetBasePosition());
				TrackingSpaceDeltaPose = BaseTransform.Inverse() * TrackingSpaceDeltaPose * BaseTransform;

				FrameSynthesisLayerInfo.type = XR_TYPE_FRAME_SYNTHESIS_INFO_EXT;
				FrameSynthesisLayerInfo.next = nullptr;
				FrameSynthesisLayerInfo.layerFlags = XR_FRAME_SYNTHESIS_INFO_REQUEST_RELAXED_FRAME_INTERVAL_BIT_EXT;
				FrameSynthesisLayerInfo.motionVectorSubImage = MotionVectorImage;
				FrameSynthesisLayerInfo.motionVectorScale = { 1.0f, 1.0f, 1.0f, 0.0f };
				FrameSynthesisLayerInfo.motionVectorOffset = {};
				FrameSynthesisLayerInfo.appSpaceDeltaPose = ToXrPose(TrackingSpaceDeltaPose);
				FrameSynthesisLayerInfo.depthSubImage = MotionVectorDepthImage;
				FrameSynthesisLayerInfo.minDepth = 0.0f;
				FrameSynthesisLayerInfo.maxDepth = 1.0f;
				FrameSynthesisLayerInfo.nearZ = FLT_MAX;
				FrameSynthesisLayerInfo.farZ = NearZ;

				Projection.next = &FrameSynthesisLayerInfo;
			}
			else if (IsExtensionEnabled(XR_FB_SPACE_WARP_EXTENSION_NAME))
			{
				XrCompositionLayerSpaceWarpInfoFB& SpaceWarpLayerInfo = PipelinedLayerStateRendering.SpaceWarpLayers[ViewIndex];

				FTransform TrackingToWorld = GetTrackingToWorldTransform();
				FTransform TrackingSpaceDeltaPose = TrackingToWorld * FrameSynthesisLastTrackingToWorld.Inverse();
				FrameSynthesisLastTrackingToWorld = TrackingToWorld;
				FTransform BaseTransform = FTransform(GetBaseOrientation(), GetBasePosition());
				TrackingSpaceDeltaPose = BaseTransform.Inverse() * TrackingSpaceDeltaPose * BaseTransform;

				SpaceWarpLayerInfo.type = XR_TYPE_COMPOSITION_LAYER_SPACE_WARP_INFO_FB;
				SpaceWarpLayerInfo.next = nullptr;
				SpaceWarpLayerInfo.layerFlags = 0;
				SpaceWarpLayerInfo.motionVectorSubImage = MotionVectorImage;
				SpaceWarpLayerInfo.appSpaceDeltaPose = ToXrPose(TrackingSpaceDeltaPose);
				SpaceWarpLayerInfo.depthSubImage = MotionVectorDepthImage;
				SpaceWarpLayerInfo.minDepth = 0.0f;
				SpaceWarpLayerInfo.maxDepth = 1.0f;
				SpaceWarpLayerInfo.nearZ = FLT_MAX;
				SpaceWarpLayerInfo.farZ = NearZ;

				Projection.next = &SpaceWarpLayerInfo;
			}
			else
			{
				checkNoEntry();
			}
		}

		for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
		{
			Projection.next = Module->OnBeginProjectionView(Session, 0, ViewIndex, Projection.next);
		}
	}

	// We use RHICmdList directly, though eventually, we might want to schedule on GraphBuilder
	GraphBuilder.RHICmdList.EnqueueLambda([this, LayerState = PipelinedLayerStateRendering](FRHICommandListImmediate&)
	{
		PipelinedLayerStateRHI = LayerState;
	});
}

bool FOpenXRHMD::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	// Don't activate the SVE if xr is being used for tracking only purposes
	static const bool bXrTrackingOnly = FParse::Param(FCommandLine::Get(), TEXT("xrtrackingonly"));

	return FHMDSceneViewExtension::IsActiveThisFrame_Internal(Context) && !bXrTrackingOnly;
}

bool CheckPlatformDepthExtensionSupport(const XrInstanceProperties& InstanceProps)
{
	if (!CVarOpenXRAllowDepthLayer.GetValueOnAnyThread())
	{
		return false;
	}

	if (FCStringAnsi::Strstr(InstanceProps.runtimeName, "SteamVR/OpenXR") && RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		return false;
	}
	return true;
}

bool CheckPlatformAcquireOnAnyThreadSupport(const XrInstanceProperties& InstanceProps)
{
	int32 AcquireMode = CVarOpenXRAcquireMode.GetValueOnAnyThread();
	if (AcquireMode > 0)
	{
		return AcquireMode == 1;
	}
	else if (RHIGetInterfaceType() != ERHIInterfaceType::Vulkan || FCStringAnsi::Strstr(InstanceProps.runtimeName, "Oculus"))
	{
		return true;
	}
	return false;
}

FOpenXRHMD::FOpenXRHMD(const FAutoRegister& AutoRegister, XrInstance InInstance, TRefCountPtr<FOpenXRRenderBridge>& InRenderBridge, TArray<const char*> InEnabledExtensions, TArray<IOpenXRExtensionPlugin*> InExtensionPlugins, IARSystemSupport* ARSystemSupport, EOpenXRAPIVersion InOpenXRAPIVersion)
	: FHeadMountedDisplayBase(ARSystemSupport)
	, FHMDSceneViewExtension(AutoRegister)
	, FOpenXRAssetManager(InInstance, this)
	, bStereoEnabled(false)
	, bIsRunning(false)
	, bIsReady(false)
	, bIsRendering(false)
	, bIsSynchronized(false)
	, bShouldWait(true)
	, bIsExitingSessionByxrRequestExitSession(false)
	, bNeedReBuildOcclusionMesh(true)
	, bIsMobileMultiViewEnabled(false)
	, bSupportsHandTracking(false)
	, bIsStandaloneStereoOnlyDevice(false)
	, bRuntimeRequiresRHIContext(false)
	, bIsTrackingOnlySession(false)
	, CurrentSessionState(XR_SESSION_STATE_UNKNOWN)
	, EnabledExtensions(std::move(InEnabledExtensions))
	, InputModule(nullptr)
	, ExtensionPlugins(std::move(InExtensionPlugins))
	, Instance(InInstance)
	, OpenXRAPIVersion(InOpenXRAPIVersion)
	, System(XR_NULL_SYSTEM_ID)
	, Session(XR_NULL_HANDLE)
	, LocalSpace(XR_NULL_HANDLE)
	, LocalFloorSpace(XR_NULL_HANDLE)
	, StageSpace(XR_NULL_HANDLE)
	, CustomSpace(XR_NULL_HANDLE)
	, TrackingSpaceType(XR_REFERENCE_SPACE_TYPE_STAGE)
	, SelectedViewConfigurationType(XR_VIEW_CONFIGURATION_TYPE_MAX_ENUM)
	, SelectedEnvironmentBlendMode(XR_ENVIRONMENT_BLEND_MODE_MAX_ENUM)
	, RenderBridge(InRenderBridge)
	, RendererModule(nullptr)
	, LastRequestedColorSwapchainFormat(0)
	, LastActualColorSwapchainFormat(0)
	, LastRequestedDepthSwapchainFormat(PF_DepthStencil)
	, bTrackingSpaceInvalid(true)
	, bUseCustomReferenceSpace(false)
	, BaseOrientation(FQuat::Identity)
	, BasePosition(FVector::ZeroVector)
	, LayerColorScale{ 1.0f, 1.0f, 1.0f, 1.0f }
	, LayerColorBias{ 0.0f, 0.0f, 0.0f, 0.0f }
	, bxrGetSystemPropertiesSuccessful(false)
	, RecommendedMotionVectorTextureSize(0, 0)
{
	check(OpenXRAPIVersion != EOpenXRAPIVersion::V_INVALID);

	InstanceProperties = { XR_TYPE_INSTANCE_PROPERTIES, nullptr };
	XR_ENSURE(xrGetInstanceProperties(Instance, &InstanceProperties));
	InstanceProperties.runtimeName[XR_MAX_RUNTIME_NAME_SIZE - 1] = 0; // Ensure the name is null terminated.

	bDepthExtensionSupported = IsExtensionEnabled(XR_KHR_COMPOSITION_LAYER_DEPTH_EXTENSION_NAME) && CheckPlatformDepthExtensionSupport(InstanceProperties);
	bHiddenAreaMaskSupported = IsExtensionEnabled(XR_KHR_VISIBILITY_MASK_EXTENSION_NAME) && !FCStringAnsi::Strstr(InstanceProperties.runtimeName, "Oculus");
	bViewConfigurationFovSupported = IsExtensionEnabled(XR_EPIC_VIEW_CONFIGURATION_FOV_EXTENSION_NAME);
	bCompositionLayerColorScaleBiasSupported = IsExtensionEnabled(XR_KHR_COMPOSITION_LAYER_COLOR_SCALE_BIAS_EXTENSION_NAME);
	bSupportsHandTracking = IsExtensionEnabled(XR_EXT_HAND_TRACKING_EXTENSION_NAME);
	bSpaceAccelerationSupported = IsExtensionEnabled(XR_EPIC_SPACE_ACCELERATION_NAME);
	bIsAcquireOnAnyThreadSupported = CheckPlatformAcquireOnAnyThreadSupport(InstanceProperties);
	bUseWaitCountToAvoidExtraXrBeginFrameCalls = CVarOpenXRUseWaitCountToAvoidExtraXrBeginFrameCalls.GetValueOnAnyThread();
	
	ReconfigureForShaderPlatform(GMaxRHIShaderPlatform);

	bFoveationExtensionSupported = 
		IsExtensionEnabled(XR_FB_SWAPCHAIN_UPDATE_STATE_EXTENSION_NAME) &&
		IsExtensionEnabled(XR_FB_FOVEATION_EXTENSION_NAME) &&
		IsExtensionEnabled(XR_FB_FOVEATION_CONFIGURATION_EXTENSION_NAME);

	bEquirectLayersSupported = IsExtensionEnabled(XR_KHR_COMPOSITION_LAYER_EQUIRECT_EXTENSION_NAME) || IsExtensionEnabled(XR_KHR_COMPOSITION_LAYER_EQUIRECT2_EXTENSION_NAME);
	bCylinderLayersSupported = IsExtensionEnabled(XR_KHR_COMPOSITION_LAYER_CYLINDER_EXTENSION_NAME);
	bCubemapLayersSupported = IsExtensionEnabled(XR_KHR_COMPOSITION_LAYER_CUBE_EXTENSION_NAME);

#ifdef XR_USE_GRAPHICS_API_VULKAN
	bFoveationExtensionSupported &= IsExtensionEnabled(XR_FB_FOVEATION_VULKAN_EXTENSION_NAME) && GRHISupportsAttachmentVariableRateShading && GRHIVariableRateShadingImageDataType == VRSImage_Fractional;
#endif

	bLocalFloorSpaceSupported = IsOpenXRAPIVersionMet(EOpenXRAPIVersion::V_1_1) || IsExtensionEnabled(XR_EXT_LOCAL_FLOOR_EXTENSION_NAME);

 	bIsStandaloneStereoOnlyDevice |= UE::OpenXR::bStandAloneStereoOnlyDevice;
	if (bIsStandaloneStereoOnlyDevice)
	{
		UE_LOGF(LogHMD, Log, "IsStandaloneStereoOnlyDevice %i because of XR.StandAloneStereoOnlyDevice", bIsStandaloneStereoOnlyDevice)
	}
	else
	{
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		if (Module->IsStandaloneStereoOnlyDevice())
		{
			bIsStandaloneStereoOnlyDevice = true;
				UE_LOGF(LogHMD, Log, "IsStandaloneStereoOnlyDevice %i because of Plugin %ls", bIsStandaloneStereoOnlyDevice, *Module->GetDisplayName())
				break;
			}
		}
	}

	bIsTrackingOnlySession = FParse::Param(FCommandLine::Get(), TEXT("xrtrackingonly"));

	// Add a device space for the HMD without an action handle and ensure it has the correct index
	XrPath UserHead = XR_NULL_PATH;
	XR_ENSURE(xrStringToPath(Instance, "/user/head", &UserHead));
	ensure(DeviceSpaces.Emplace(XR_NULL_HANDLE, UserHead) == HMDDeviceId);

	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Module->BindExtensionPluginDelegates(*this);
		bRuntimeRequiresRHIContext |= Module->RequiresRHIContext();
	}

	FrameSynthesisLastTrackingToWorld = FTransform::Identity;

	const UOpenXRHMDSettings* Settings = GetDefault<UOpenXRHMDSettings>();
	bRuntimeFoveationSupported = bFoveationExtensionSupported && (Settings != nullptr ? Settings->bIsFBFoveationEnabled : false);
	if (bRuntimeFoveationSupported && !FBFoveationImageGenerator.IsValid())
	{
		FBFoveationImageGenerator = MakeUnique<FFBFoveationImageGenerator>(bRuntimeFoveationSupported, Instance, this, bIsMobileMultiViewEnabled);
		GVRSImageManager.RegisterExternalImageGenerator(FBFoveationImageGenerator.Get());
	}

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().AddRaw(this, &FOpenXRHMD::OnBackBufferReadyToPresent);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FOpenXRHMD::~FOpenXRHMD()
{
	if (bRuntimeFoveationSupported)
	{
		GVRSImageManager.UnregisterExternalImageGenerator(FBFoveationImageGenerator.Get());
		FBFoveationImageGenerator.Reset();
	}
	DestroySession();

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().RemoveAll(this);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FOpenXRHMD::ReconfigureForShaderPlatform(EShaderPlatform NewShaderPlatform)
{
	UE::StereoRenderUtils::FStereoShaderAspects Aspects(NewShaderPlatform);
	bIsMobileMultiViewEnabled = Aspects.IsMobileMultiViewEnabled();

	static const auto CVarPropagateAlpha = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessing.PropagateAlpha"));
	static const auto CVarPropagateAlphaMobile = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.PropagateAlpha"));
	bProjectionLayerAlphaEnabled = IsMobilePlatform(NewShaderPlatform) ? CVarPropagateAlphaMobile->GetBool() : CVarPropagateAlpha->GetBool();

	ConfiguredShaderPlatform = NewShaderPlatform;

	UE_LOGF(LogHMD, Log, "HMD configured for shader platform %ls, bIsMobileMultiViewEnabled=%d, bProjectionLayerAlphaEnabled=%d",
		*LexToString(ConfiguredShaderPlatform),
		bIsMobileMultiViewEnabled,
		bProjectionLayerAlphaEnabled
		);

	return true;
}

TArray<XrEnvironmentBlendMode> FOpenXRHMD::GetSupportedEnvironmentBlendModes() const
{
	TArray<XrEnvironmentBlendMode> BlendModes;
	uint32 BlendModeCount;
	XR_ENSURE(xrEnumerateEnvironmentBlendModes(Instance, System, SelectedViewConfigurationType, 0, &BlendModeCount, nullptr));
	// Fill the initial array with valid enum types (this will fail in the validation layer otherwise).
	BlendModes.Init(XR_ENVIRONMENT_BLEND_MODE_OPAQUE, BlendModeCount);
	XR_ENSURE(xrEnumerateEnvironmentBlendModes(Instance, System, SelectedViewConfigurationType, BlendModeCount, &BlendModeCount, BlendModes.GetData()));
	return BlendModes;
}

FOpenXRHMD::FPipelinedFrameStateAccessorReadOnly FOpenXRHMD::GetPipelinedFrameStateForThread() const
{
	// Relying on implicit selection of the RHI struct is hazardous since the RHI thread isn't always present
	check(!IsInRHIThread());

	// Opening up access to parallel rendering threads, because some frame state (e.g. GetDesiredNumberOfViews()) is started being requested on them.
	// Since the frame state is returned const from this function, this is hopefully a little bit more safe, but still prone to race conditions if the real
	// render thread at this moment is modifying the state using the non-const method. Proper resolution is tracked in UE-212224.
	if (IsInActualRenderingThread() || IsInParallelRenderingThread())
	{
		return FPipelinedFrameStateAccessorReadOnly(PipelinedFrameStateRendering, PipelinedFrameStateRenderingAccessGuard);
	}
	else
	{
		check(IsInGameThread() || IsInParallelGameThread());
		return FPipelinedFrameStateAccessorReadOnly(PipelinedFrameStateGame, PipelinedFrameStateGameAccessGuard);
	}
}

FOpenXRHMD::FPipelinedFrameStateAccessorReadWrite FOpenXRHMD::GetPipelinedFrameStateForThread()
{
	// Relying on implicit selection of the RHI struct is hazardous since the RHI thread isn't always present
	check(!IsInRHIThread());

	if (IsInActualRenderingThread() || IsInParallelRenderingThread())
	{
		return FPipelinedFrameStateAccessorReadWrite(PipelinedFrameStateRendering, PipelinedFrameStateRenderingAccessGuard);
	}
	else
	{
		check(IsInGameThread() || IsInParallelGameThread());
		return FPipelinedFrameStateAccessorReadWrite(PipelinedFrameStateGame, PipelinedFrameStateGameAccessGuard);
	}
}

void FOpenXRHMD::UpdateDeviceLocations(bool bUpdateOpenXRExtensionPlugins)
{
	SCOPED_NAMED_EVENT(UpdateDeviceLocations, FColor::Red);

	FPipelinedFrameStateAccessorReadWrite LockedPipelineState = GetPipelinedFrameStateForThread();
	FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();

	// Only update the device locations if the frame state has been predicted, which is dependent on WaitFrame success
	// Also need a valid TrackingSpace
	if (PipelineState.bXrFrameStateUpdated && PipelineState.TrackingSpace.IsValid())
	{
		FReadScopeLock Lock(DeviceMutex);
		PipelineState.DeviceLocations.SetNumZeroed(DeviceSpaces.Num());
		for (int32 DeviceIndex = 0; DeviceIndex < PipelineState.DeviceLocations.Num(); DeviceIndex++)
		{
			const FDeviceSpace& DeviceSpace = DeviceSpaces[DeviceIndex];
			XrSpaceLocation& CachedDeviceLocation = PipelineState.DeviceLocations[DeviceIndex];
			CachedDeviceLocation.type = XR_TYPE_SPACE_LOCATION;

			if (DeviceSpace.Space != XR_NULL_HANDLE)
			{
				XrSpaceLocation NewDeviceLocation = { XR_TYPE_SPACE_LOCATION };
				XrResult Result = xrLocateSpace(DeviceSpace.Space, PipelineState.TrackingSpace->Handle, PipelineState.FrameState.predictedDisplayTime, &NewDeviceLocation);
				if (Result == XR_ERROR_TIME_INVALID)
				{
					// The display time is no longer valid so set the location as invalid as well
					PipelineState.DeviceLocations[DeviceIndex].locationFlags = 0;
				}
				else if (Result != XR_SUCCESS)
				{
					PipelineState.DeviceLocations[DeviceIndex].locationFlags = 0;
					ensureMsgf(XR_SUCCEEDED(Result), TEXT("OpenXR xrLocateSpace failed with result %s.  No pose fetched."), OpenXRResultToString(Result)); \
				}
				else
				{
					// Clear the location tracked bits
					CachedDeviceLocation.locationFlags &= ~(XR_SPACE_LOCATION_POSITION_TRACKED_BIT | XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT);
					if (NewDeviceLocation.locationFlags & (XR_SPACE_LOCATION_POSITION_VALID_BIT))
					{
						CachedDeviceLocation.pose.position = NewDeviceLocation.pose.position;
						CachedDeviceLocation.locationFlags |= (NewDeviceLocation.locationFlags & (XR_SPACE_LOCATION_POSITION_TRACKED_BIT | XR_SPACE_LOCATION_POSITION_VALID_BIT));
					}
					if (NewDeviceLocation.locationFlags & (XR_SPACE_LOCATION_ORIENTATION_VALID_BIT))
					{
						CachedDeviceLocation.pose.orientation = NewDeviceLocation.pose.orientation;
						CachedDeviceLocation.locationFlags |= (NewDeviceLocation.locationFlags & (XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT | XR_SPACE_LOCATION_ORIENTATION_VALID_BIT));
					}
				}
			}
			else
			{
				// Ensure the location flags are zeroed out so the pose is detected as invalid
				CachedDeviceLocation.locationFlags = 0;
			}
		}

		if (bUpdateOpenXRExtensionPlugins)
		{
			for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
			{
				Module->UpdateDeviceLocations(Session, PipelineState.FrameState.predictedDisplayTime, PipelineState.TrackingSpace->Handle);
			}
		}
	}
}

void FOpenXRHMD::EnumerateViews(FPipelinedFrameState& PipelineState)
{
	SCOPED_NAMED_EVENT(EnumerateViews, FColor::Red);

	// Enumerate the viewport configuration views
	uint32 ViewConfigCount = 0;
	TArray<XrViewConfigurationViewFovEPIC> ViewFov;
	TArray<XrFrameSynthesisConfigViewEXT> ViewFrameSynthesis;

	XR_ENSURE(xrEnumerateViewConfigurationViews(Instance, System, SelectedViewConfigurationType, 0, &ViewConfigCount, nullptr));
	ViewFov.SetNum(ViewConfigCount);
	ViewFrameSynthesis.SetNum(ViewConfigCount);
	PipelineState.ViewConfigs.Empty(ViewConfigCount);
	for (uint32 ViewIndex = 0; ViewIndex < ViewConfigCount; ViewIndex++)
	{
		XrViewConfigurationView View;
		View.type = XR_TYPE_VIEW_CONFIGURATION_VIEW;

		ViewFov[ViewIndex].type = XR_TYPE_VIEW_CONFIGURATION_VIEW_FOV_EPIC;
		ViewFov[ViewIndex].next = nullptr;
		View.next = bViewConfigurationFovSupported ? &ViewFov[ViewIndex] : nullptr;

		if (IsExtensionEnabled(XR_EXT_FRAME_SYNTHESIS_EXTENSION_NAME))
		{
			check(!IsExtensionEnabled(XR_FB_SPACE_WARP_EXTENSION_NAME));

			ViewFrameSynthesis[ViewIndex].type = XR_TYPE_FRAME_SYNTHESIS_CONFIG_VIEW_EXT;
			ViewFrameSynthesis[ViewIndex].next = View.next;
			View.next = &ViewFrameSynthesis[ViewIndex];
		}

		for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
		{
			View.next = Module->OnEnumerateViewConfigurationViews(Instance, System, SelectedViewConfigurationType, ViewIndex, View.next);
		}

		XrFrameSynthesisConfigViewEXT FrameSynthesisViewConfig = { XR_TYPE_FRAME_SYNTHESIS_CONFIG_VIEW_EXT };
		
		PipelineState.ViewConfigs.Add(View);
	}
	XR_ENSURE(xrEnumerateViewConfigurationViews(Instance, System, SelectedViewConfigurationType, ViewConfigCount, &ViewConfigCount, PipelineState.ViewConfigs.GetData()));

	if (Session)
	{
		LocateViews(PipelineState, true);

		check(PipelineState.bXrFrameStateUpdated);
	}
	else if (bViewConfigurationFovSupported)
	{
		// We can't locate the views yet, but we can already retrieve their field-of-views
		PipelineState.Views.SetNum(PipelineState.ViewConfigs.Num());
		for (int ViewIndex = 0; ViewIndex < PipelineState.Views.Num(); ViewIndex++)
		{
			XrView& View = PipelineState.Views[ViewIndex];
			View.type = XR_TYPE_VIEW;
			View.next = nullptr;
			View.fov = ViewFov[ViewIndex].recommendedFov;
			View.pose = ToXrPose(FTransform::Identity);
		}
	}
	else
	{
		// Ensure the views have sane values before we locate them
		PipelineState.Views.SetNum(PipelineState.ViewConfigs.Num());
		for (XrView& View : PipelineState.Views)
		{
			View.type = XR_TYPE_VIEW;
			View.next = nullptr;
			View.fov = XrFovf{ -PI / 4.0f, PI / 4.0f, PI / 4.0f, -PI / 4.0f };
			View.pose = ToXrPose(FTransform::Identity);
		}
	}

	if (IsExtensionEnabled(XR_EXT_FRAME_SYNTHESIS_EXTENSION_NAME) && PipelineState.ViewConfigs.Num())
	{
		// Variable size between views is not currently supported
		XrFrameSynthesisConfigViewEXT& FrameSynthConfig = ViewFrameSynthesis[0];
		RecommendedMotionVectorTextureSize = FIntPoint(FrameSynthConfig.recommendedMotionVectorImageRectWidth, FrameSynthConfig.recommendedMotionVectorImageRectHeight);
		ensure(RecommendedMotionVectorTextureSize.X && RecommendedMotionVectorTextureSize.Y);
	}
}

void FOpenXRHMD::BuildOcclusionMeshes()
{
	SCOPED_NAMED_EVENT(BuildOcclusionMeshes, FColor::Red);

	uint32_t ViewCount = 0;
	XR_ENSURE(xrEnumerateViewConfigurationViews(Instance, System, SelectedViewConfigurationType, 0, &ViewCount, nullptr));
	HiddenAreaMeshes.SetNum(ViewCount);
	VisibleAreaMeshes.SetNum(ViewCount);

	bool bAnyViewSucceeded = false;

	for (uint32_t View = 0; View < ViewCount; ++View)
	{
		if (BuildOcclusionMesh(XR_VISIBILITY_MASK_TYPE_VISIBLE_TRIANGLE_MESH_KHR, View, VisibleAreaMeshes[View]) &&
			BuildOcclusionMesh(XR_VISIBILITY_MASK_TYPE_HIDDEN_TRIANGLE_MESH_KHR, View, HiddenAreaMeshes[View]))
		{
			bAnyViewSucceeded = true;
		}
	}

	if (!bAnyViewSucceeded)
	{
		UE_LOGF(LogHMD, Error, "Failed to create all visibility mask meshes for device/views. Abandoning visibility mask.");

		HiddenAreaMeshes.Empty();
		VisibleAreaMeshes.Empty();
	}

	bNeedReBuildOcclusionMesh = false;
}

bool FOpenXRHMD::BuildOcclusionMesh(XrVisibilityMaskTypeKHR Type, int View, FHMDViewMesh& Mesh)
{
	FReadScopeLock Lock(SessionHandleMutex);
	if (!Session)
	{
		return false;
	}

	PFN_xrGetVisibilityMaskKHR GetVisibilityMaskKHR;
	XR_ENSURE(xrGetInstanceProcAddr(Instance, "xrGetVisibilityMaskKHR", (PFN_xrVoidFunction*)&GetVisibilityMaskKHR));

	XrVisibilityMaskKHR VisibilityMask = { XR_TYPE_VISIBILITY_MASK_KHR };
	XR_ENSURE(GetVisibilityMaskKHR(Session, SelectedViewConfigurationType, View, Type, &VisibilityMask));

	if (VisibilityMask.indexCountOutput == 0)
	{
		// Runtime doesn't have a valid mask for this view
		return false;
	}
	if (!VisibilityMask.indexCountOutput || (VisibilityMask.indexCountOutput % 3) != 0 || VisibilityMask.vertexCountOutput == 0)
	{
		UE_LOGF(LogHMD, Error, "Visibility Mask Mesh returned from runtime is invalid.");
		return false;
	}

	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	const FRHIBufferCreateDesc VertexCreateDesc =
		FRHIBufferCreateDesc::CreateVertex<FFilterVertex>(TEXT("FOpenXRHMD"), VisibilityMask.vertexCountOutput)
		.AddUsage(EBufferUsageFlags::Static)
		.SetInitActionInitializer()
		.DetermineInitialState();
	TRHIBufferInitializer<FFilterVertex> VertexInitialData = RHICmdList.CreateBufferInitializer(VertexCreateDesc);

	const FRHIBufferCreateDesc IndexCreateDesc =
		FRHIBufferCreateDesc::CreateIndex<uint32>(TEXT("FOpenXRHMD"), VisibilityMask.indexCountOutput)
		.AddUsage(EBufferUsageFlags::Static)
		.SetInitActionInitializer()
		.DetermineInitialState();
	TRHIBufferInitializer<uint32> IndexInitialData = RHICmdList.CreateBufferInitializer(IndexCreateDesc);

	TUniquePtr<XrVector2f[]> const OutVertices = MakeUnique<XrVector2f[]>(VisibilityMask.vertexCountOutput);
	TUniquePtr<uint32[]> const OutIndices = MakeUnique<uint32[]>(VisibilityMask.indexCountOutput);

	VisibilityMask.vertexCapacityInput = VisibilityMask.vertexCountOutput;
	VisibilityMask.indexCapacityInput = VisibilityMask.indexCountOutput;
	VisibilityMask.indices = OutIndices.Get();
	VisibilityMask.vertices = OutVertices.Get();

	GetVisibilityMaskKHR(Session, SelectedViewConfigurationType, View, Type, &VisibilityMask);

	IndexInitialData.WriteArray(MakeConstArrayView(OutIndices.Get(), VisibilityMask.indexCountOutput));

	// We need to apply the eye's projection matrix to each vertex
	FMatrix Projection = GetStereoProjectionMatrix(View);

	ensure(VisibilityMask.vertexCapacityInput == VisibilityMask.vertexCountOutput);
	ensure(VisibilityMask.indexCapacityInput == VisibilityMask.indexCountOutput);

	for (uint32 VertexIndex = 0; VertexIndex < VisibilityMask.vertexCountOutput; ++VertexIndex)
	{
		FFilterVertex Vertex = { FVector4f(ForceInitToZero), FVector2f(ForceInitToZero) };
		FVector Position(OutVertices[VertexIndex].x, OutVertices[VertexIndex].y, 1.0f);

		Vertex.Position = (FVector4f)Projection.TransformPosition(Position); // LWC_TODO: precision loss

		if (Type == XR_VISIBILITY_MASK_TYPE_VISIBLE_TRIANGLE_MESH_KHR)
		{
			// For the visible-area mesh, this will be consumed by the post-process pipeline, so set up coordinates in the space they expect
			// (x and y range from 0-1, origin bottom-left, z at the far plane).
			Vertex.Position.X = Vertex.Position.X / 2.0f + .5f;
			Vertex.Position.Y = Vertex.Position.Y / -2.0f + .5f;
			Vertex.Position.Z = 0.0f;
			Vertex.Position.W = 1.0f;
		}

		Vertex.UV.X = Vertex.Position.X;
		Vertex.UV.Y = Vertex.Position.Y;

		VertexInitialData.WriteValueAtIndex(VertexIndex, Vertex);
	}

	Mesh.VertexBufferRHI = VertexInitialData.Finalize();
	Mesh.IndexBufferRHI = IndexInitialData.Finalize();

	Mesh.NumIndices = VisibilityMask.indexCountOutput;
	Mesh.NumVertices = VisibilityMask.vertexCountOutput;
	Mesh.NumTriangles = Mesh.NumIndices / 3;

	return true;
}

#if WITH_EDITOR
// Show a warning that the editor will require a restart
void ShowRestartWarning(const FText& Title)
{
	if (EAppReturnType::Ok == FMessageDialog::Open(EAppMsgType::OkCancel,
		LOCTEXT("EditorRestartMsg", "The OpenXR runtime requires switching to a different GPU adapter, this requires an editor restart. Do you wish to restart now (you will be prompted to save any changes)?"),
		Title))
	{
		FUnrealEdMisc::Get().RestartEditor(false);
	}
}
#endif

bool FOpenXRHMD::PopulateAnalyticsAttributes(TArray<FAnalyticsEventAttribute>& EventAttributes)
{
	if (!FHeadMountedDisplayBase::PopulateAnalyticsAttributes(EventAttributes))
	{
		return false;
	}

	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("xrGetSystemPropertiesSuccessful"), bxrGetSystemPropertiesSuccessful));
	return true;
}

bool FOpenXRHMD::AcquireSystemIdAndProperties()
{
	// system does not seem to be governed by the session handle lock as it can be reset in OnStartGameFrame outside of the lock
	System = IOpenXRHMDModule::Get().GetSystemId();
	if (!System)
	{
		return false;
	}

	// Retrieve system properties and check for feature support
	SystemProperties = XrSystemProperties{ XR_TYPE_SYSTEM_PROPERTIES };
	void* Tail = &SystemProperties;
	XrSystemHandTrackingPropertiesEXT HandTrackingSystemProperties = { XR_TYPE_SYSTEM_HAND_TRACKING_PROPERTIES_EXT };
	if (IsExtensionEnabled(XR_EXT_HAND_TRACKING_EXTENSION_NAME))
	{
		OpenXR::AppendChainStruct(Tail, &HandTrackingSystemProperties);
	}
	XrSystemUserPresencePropertiesEXT UserPresenceSystemProperties = { XR_TYPE_SYSTEM_USER_PRESENCE_PROPERTIES_EXT };
	if (IsExtensionEnabled(XR_EXT_USER_PRESENCE_EXTENSION_NAME))
	{
		OpenXR::AppendChainStruct(Tail, &UserPresenceSystemProperties);
	}
	XrSystemSpaceWarpPropertiesFB SpaceWarpViewConfig = { XR_TYPE_SYSTEM_SPACE_WARP_PROPERTIES_FB };
	if (IsExtensionEnabled(XR_FB_SPACE_WARP_EXTENSION_NAME))
	{
		check(!IsExtensionEnabled(XR_EXT_FRAME_SYNTHESIS_EXTENSION_NAME));
		OpenXR::AppendChainStruct(Tail, &SpaceWarpViewConfig);
	}

	XrResult GetSystemPropsResult = xrGetSystemProperties(Instance, System, &SystemProperties);
	XR_ENSURE(GetSystemPropsResult);
	bxrGetSystemPropertiesSuccessful = (GetSystemPropsResult == XR_SUCCESS);

	// Some runtimes aren't compliant with their number of layers supported.
	// We support a fallback by emulating non-facelocked layers
	bLayerSupportOpenXRCompliant = SystemProperties.graphicsProperties.maxLayerCount >= XR_MIN_COMPOSITION_LAYERS_SUPPORTED;
	bSupportsHandTracking = HandTrackingSystemProperties.supportsHandTracking == XR_TRUE;
	bUserPresenceSupported = UserPresenceSystemProperties.supportsUserPresence == XR_TRUE;

	if (IsExtensionEnabled(XR_FB_SPACE_WARP_EXTENSION_NAME))
	{
		RecommendedMotionVectorTextureSize = FIntPoint(SpaceWarpViewConfig.recommendedMotionVectorImageRectWidth, SpaceWarpViewConfig.recommendedMotionVectorImageRectHeight);
		check(RecommendedMotionVectorTextureSize.X && RecommendedMotionVectorTextureSize.Y);
	}

	// We have built a linked list to function-scope structs and attached it to this member variable, those pointers will be bad after this function exits.
	// Currently we never want to use them again, so we can just null out the next pointer.
	SystemProperties.next = nullptr;

	return true;
}

int32 FOpenXRHMD::GetNumHeaders() const
{
	int32 NumHeaders = 0;
	for (const FOpenXRLayer& Layer : NativeLayers)
	{
		NumHeaders += Layer.GetNumHeaders();
	}
	return NumHeaders;
}

bool FOpenXRHMD::OnStereoStartup()
{
	FWriteScopeLock Lock(SessionHandleMutex);

	bIsExitingSessionByxrRequestExitSession = false;  // clear in case we requested exit for a previous session, but it ended in some other way before that happened.

	if (Session)
	{
		return false;
	}

	if (!AcquireSystemIdAndProperties())
	{
		UE_LOGF(LogHMD, Error, "Failed to get an OpenXR system, please check that you have a VR headset connected.");
		return false;
	}

	// Enumerate the viewport configurations
	uint32 ConfigurationCount;
	TArray<XrViewConfigurationType> ViewConfigTypes;
	XR_ENSURE(xrEnumerateViewConfigurations(Instance, System, 0, &ConfigurationCount, nullptr));
	// Fill the initial array with valid enum types (this will fail in the validation layer otherwise).
	ViewConfigTypes.Init(XR_VIEW_CONFIGURATION_TYPE_PRIMARY_MONO, ConfigurationCount);
	XR_ENSURE(xrEnumerateViewConfigurations(Instance, System, ConfigurationCount, &ConfigurationCount, ViewConfigTypes.GetData()));
	XrViewConfigurationType PreferredFallbackType = ViewConfigTypes[0];
	
	// Filter to supported configurations only
	ViewConfigTypes = ViewConfigTypes.FilterByPredicate([&](XrViewConfigurationType Type) 
		{ 
			return SupportedViewConfigurations.Contains(Type); 
		});

	// If we've specified a view configuration override and it's available, try to use that.
	// Otherwise select the first view configuration returned by the runtime that is supported.
	// This is the view configuration preferred by the runtime.
	XrViewConfigurationType* PreferredViewConfiguration = ViewConfigTypes.FindByPredicate([&](XrViewConfigurationType Type)
		{
			return Type == CVarOpenXRPreferredViewConfiguration.GetValueOnGameThread();
		});

	if (PreferredViewConfiguration)
	{
		SelectedViewConfigurationType = *PreferredViewConfiguration;
	}
	else if (ViewConfigTypes.Num() > 0)
	{
		SelectedViewConfigurationType = ViewConfigTypes[0];
	}

	// If there is no supported view configuration type, use the first option as a last resort.
	if (!ensure(SelectedViewConfigurationType != XR_VIEW_CONFIGURATION_TYPE_MAX_ENUM))
	{
		UE_LOGF(LogHMD, Error, "No compatible view configuration type found, falling back to runtime preferred type.");
		SelectedViewConfigurationType = PreferredFallbackType;
	}

	// Enumerate the views we will be simulating with.
	EnumerateViews(PipelinedFrameStateGame);

	for (const XrViewConfigurationView& Config : PipelinedFrameStateGame.ViewConfigs)
	{
		const float WidthDensityMax = float(Config.maxImageRectWidth) / Config.recommendedImageRectWidth;
		const float HeightDensitymax = float(Config.maxImageRectHeight) / Config.recommendedImageRectHeight;
		const float PerViewPixelDensityMax = FMath::Min(WidthDensityMax, HeightDensitymax);
		RuntimePixelDensityMax = FMath::Min(RuntimePixelDensityMax, PerViewPixelDensityMax);
	}

	// Select the first blend mode returned by the runtime - as per spec, environment blend modes should be in order from highest to lowest runtime preference
	{
		TArray<XrEnvironmentBlendMode> BlendModes = GetSupportedEnvironmentBlendModes();
		if (!BlendModes.IsEmpty())
		{
			SelectedEnvironmentBlendMode = BlendModes[0];
		}
	}

	// Give the all frame states the same initial values.
	PipelinedFrameStateRHI = PipelinedFrameStateRendering = PipelinedFrameStateGame;

	XrSessionCreateInfo SessionInfo;
	SessionInfo.type = XR_TYPE_SESSION_CREATE_INFO;
	SessionInfo.next = nullptr;
	if (RenderBridge.IsValid())
	{
		SessionInfo.next = RenderBridge->GetGraphicsBinding(System);
		if (!SessionInfo.next)
		{
			UE_LOGF(LogHMD, Warning, "Failed to get an OpenXR graphics binding, editor restart required.");
#if WITH_EDITOR
			ShowRestartWarning(LOCTEXT("EditorRestartMsg_Title", "Editor Restart Required"));
#endif
			return false;
		}
	}
	SessionInfo.createFlags = 0;
	SessionInfo.systemId = System;
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		SessionInfo.next = Module->OnCreateSession(Instance, System, SessionInfo.next);
	}

	if (!XR_ENSURE(xrCreateSession(Instance, &SessionInfo, &Session)))
	{
		UE_LOGF(LogHMD, Warning, "xrCreateSession failed.")
		return false;
	}

	UE_LOGF(LogHMD, Verbose, "xrCreateSession created %llu", Session);

	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Module->PostCreateSession(Session);
	}

	uint32_t ReferenceSpacesCount;
	XR_ENSURE(xrEnumerateReferenceSpaces(Session, 0, &ReferenceSpacesCount, nullptr));

	TArray<XrReferenceSpaceType> ReferenceSpaces;
	ReferenceSpaces.SetNum(ReferenceSpacesCount);
	// Initialize spaces array with valid enum values (avoid triggering validation error).
	for (auto& SpaceIter : ReferenceSpaces)
		SpaceIter = XR_REFERENCE_SPACE_TYPE_VIEW;
	XR_ENSURE(xrEnumerateReferenceSpaces(Session, (uint32_t)ReferenceSpaces.Num(), &ReferenceSpacesCount, ReferenceSpaces.GetData()));
	ensure(ReferenceSpacesCount == ReferenceSpaces.Num());

	XrSpace HmdSpace = XR_NULL_HANDLE;
	XrReferenceSpaceCreateInfo SpaceInfo;
	ensure(ReferenceSpaces.Contains(XR_REFERENCE_SPACE_TYPE_VIEW));
	SpaceInfo.type = XR_TYPE_REFERENCE_SPACE_CREATE_INFO;
	SpaceInfo.next = nullptr;
	SpaceInfo.referenceSpaceType = XR_REFERENCE_SPACE_TYPE_VIEW;
	SpaceInfo.poseInReferenceSpace = ToXrPose(FTransform::Identity);
	XR_ENSURE(xrCreateReferenceSpace(Session, &SpaceInfo, &HmdSpace));
	{
		FWriteScopeLock DeviceLock(DeviceMutex);
		DeviceSpaces[HMDDeviceId].Space = HmdSpace;
	}

	ensure(ReferenceSpaces.Contains(XR_REFERENCE_SPACE_TYPE_LOCAL));
	SpaceInfo.referenceSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL;
	XR_ENSURE(xrCreateReferenceSpace(Session, &SpaceInfo, &LocalSpace));

	if(ReferenceSpaces.Contains(XR_REFERENCE_SPACE_TYPE_LOCAL_FLOOR))
	{
		SpaceInfo.referenceSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL_FLOOR;
		XR_ENSURE(xrCreateReferenceSpace(Session, &SpaceInfo, &LocalFloorSpace));
	}

	if (ReferenceSpaces.Contains(XR_REFERENCE_SPACE_TYPE_STAGE))
	{
		SpaceInfo.referenceSpaceType = TrackingSpaceType;
		XR_ENSURE(xrCreateReferenceSpace(Session, &SpaceInfo, &StageSpace));
	}

	bUseCustomReferenceSpace = false;
	XrReferenceSpaceType CustomReferenceSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL;
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		if (Module->UseCustomReferenceSpaceType(CustomReferenceSpaceType))
		{
			bUseCustomReferenceSpace = true;
			break;
		}
	}

	// If a custom reference space is desired, try to use that.
	// Otherwise use the currently selected reference space.
	if (bUseCustomReferenceSpace && ReferenceSpaces.Contains(CustomReferenceSpaceType))
	{
		TrackingSpaceType = CustomReferenceSpaceType;
		SpaceInfo.referenceSpaceType = TrackingSpaceType;
		XR_ENSURE(xrCreateReferenceSpace(Session, &SpaceInfo, &CustomSpace));
	}
	else if (ReferenceSpaces.Contains(XR_REFERENCE_SPACE_TYPE_STAGE))
	{
		TrackingSpaceType = XR_REFERENCE_SPACE_TYPE_STAGE;
	}
	else if (ReferenceSpaces.Contains(XR_REFERENCE_SPACE_TYPE_LOCAL_FLOOR))
	{
		TrackingSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL_FLOOR;
	}
	else
	{
		ensure(ReferenceSpaces.Contains(XR_REFERENCE_SPACE_TYPE_LOCAL));
		TrackingSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL;
	}

	// Create initial tracking space
	BaseOrientation = FQuat::Identity;
	BasePosition = FVector::ZeroVector;
	PipelinedFrameStateGame.TrackingSpace = MakeShared<FTrackingSpace>(TrackingSpaceType);
	PipelinedFrameStateGame.TrackingSpace->CreateSpace(Session);

	// Create action spaces for all devices
	{
		FWriteScopeLock DeviceLock(DeviceMutex);
		for (FDeviceSpace& DeviceSpace : DeviceSpaces)
		{
			DeviceSpace.CreateSpace(Session);
		}
	}

	if (RenderBridge.IsValid())
	{
		RenderBridge->SetOpenXRHMD(this);
	}

	// grab a pointer to the renderer module for displaying our mirror window
	static const FName RendererModuleName("Renderer");
	RendererModule = FModuleManager::GetModulePtr<IRendererModule>(RendererModuleName);

	bool bUseExtensionSpectatorScreenController = false;
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		bUseExtensionSpectatorScreenController = Module->GetSpectatorScreenController(this, SpectatorScreenController);
		if (bUseExtensionSpectatorScreenController)
		{
			break;
		}
	}

	if (!bUseExtensionSpectatorScreenController && !bIsStandaloneStereoOnlyDevice)
	{
		SpectatorScreenController = MakeUnique<FDefaultSpectatorScreenController>(this);
		UE_LOGF(LogHMD, Verbose, "OpenXR using base spectator screen.");
	}
	else
	{
		if (SpectatorScreenController == nullptr)
		{
			UE_LOGF(LogHMD, Verbose, "OpenXR disabling spectator screen.");
		}
		else
		{
			UE_LOGF(LogHMD, Verbose, "OpenXR using extension spectator screen.");
		}
	}

	return true;
}

bool FOpenXRHMD::OnStereoTeardown()
{
	XrResult Result = XR_ERROR_SESSION_NOT_RUNNING;
	{
		FReadScopeLock Lock(SessionHandleMutex);
		if (Session != XR_NULL_HANDLE)
		{
			UE_LOGF(LogHMD, Verbose, "FOpenXRHMD::OnStereoTeardown() calling xrRequestExitSession");
			bIsExitingSessionByxrRequestExitSession = true;
			Result = xrRequestExitSession(Session);
		}
	}

	if (Result == XR_ERROR_SESSION_NOT_RUNNING)
	{
		// Session was never running - most likely PIE without putting the headset on.
		DestroySession();
	}
	else
	{
		XR_ENSURE_WITH_CALLINFO(Result, "xrRequestExitSession");
	}

	FCoreDelegates::VRHeadsetRecenter.RemoveAll(this);

	return true;
}

void FOpenXRHMD::DestroySession()
{
	// FlushRenderingCommands must be called outside of SessionLock since some rendering threads will also lock this mutex.
	FlushRenderingCommands();

	// Clear all the tracked devices
	ResetTrackedDevices();

	FWriteScopeLock SessionLock(SessionHandleMutex);

	if (Session != XR_NULL_HANDLE)
	{
		for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
		{
			Module->OnDestroySession(Session);
		}

		InputModule->OnDestroySession();

		// We need to reset all swapchain references to ensure there are no attempts
		// to destroy swapchain handles after the session is already destroyed.
		NativeLayers.Reset();
		BackgroundCompositedEmulatedLayers.Reset();
		EmulatedFaceLockedLayers.Reset();
		VisibleLayerIds.Reset();

		PipelinedLayerStateRendering.ColorSwapchain.Reset();
		PipelinedLayerStateRendering.DepthSwapchain.Reset();
		PipelinedLayerStateRendering.NativeLayerSwapchains.Reset();
		PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain.Reset();

		// TODO: Once we handle OnFinishRendering_RHIThread + StopSession interactions
		// properly, we can release these shared pointers in that function, and use
		// `ensure` here to make sure these are released.
		PipelinedLayerStateRHI.ColorSwapchain.Reset();
		PipelinedLayerStateRHI.DepthSwapchain.Reset();
		PipelinedLayerStateRHI.NativeLayerSwapchains.Reset();
		PipelinedLayerStateRHI.EmulatedLayerState.EmulationSwapchain.Reset();

		PipelinedFrameStateGame.TrackingSpace.Reset();
		PipelinedFrameStateRendering.TrackingSpace.Reset();
		PipelinedFrameStateRHI.TrackingSpace.Reset();
		bTrackingSpaceInvalid = true;

		// Reset the frame state.
		PipelinedFrameStateGame.bXrFrameStateUpdated = false;
		PipelinedFrameStateGame.FrameState = XrFrameState{ XR_TYPE_FRAME_STATE };
		PipelinedFrameStateRendering.bXrFrameStateUpdated = false;
		PipelinedFrameStateRendering.FrameState = XrFrameState{ XR_TYPE_FRAME_STATE };
		PipelinedFrameStateRHI.bXrFrameStateUpdated = false;
		PipelinedFrameStateRHI.FrameState = XrFrameState{ XR_TYPE_FRAME_STATE };

		// VRFocus must be reset so FWindowsApplication::PollGameDeviceState does not incorrectly short-circuit.
		FApp::SetUseVRFocus(false);
		FApp::SetHasVRFocus(false);

		// Destroy device and reference spaces, they will be recreated
		// when the session is created again.
		{
			FReadScopeLock DeviceLock(DeviceMutex);
			for (FDeviceSpace& Device : DeviceSpaces)
			{
				Device.DestroySpace();
			}
		}

		// Close the session now we're allowed to.
		XR_ENSURE(xrDestroySession(Session));
		Session = XR_NULL_HANDLE;
		CurrentSessionState = XR_SESSION_STATE_UNKNOWN;
		UE_LOGF(LogHMD, Verbose, "Session state switched to XR_SESSION_STATE_UNKNOWN by DestroySession()");
		bStereoEnabled = false;
		bIsReady = false;
		bIsRunning = false;
		bIsRendering = false;
		bIsSynchronized = false;
		bNeedReBuildOcclusionMesh = true;
	}
}
int32 FOpenXRHMD::AddTrackedDevice(XrAction Action, XrPath Path)
{
	return AddTrackedDevice(Action, Path, XR_NULL_PATH);
}
int32 FOpenXRHMD::AddTrackedDevice(XrAction Action, XrPath Path, XrPath SubactionPath)
{
	FWriteScopeLock DeviceLock(DeviceMutex);

	// Ensure the HMD device is already emplaced
	ensure(DeviceSpaces.Num() > 0);

	int32 DeviceId = DeviceSpaces.Emplace(Action, Path, SubactionPath);

	//FReadScopeLock Lock(SessionHandleMutex); // This is called from StartSession(), which already has this lock.
	if (Session)
	{
		DeviceSpaces[DeviceId].CreateSpace(Session);
	}

	return DeviceId;
}

void FOpenXRHMD::ResetTrackedDevices()
{
	FWriteScopeLock DeviceLock(DeviceMutex);

	// Index 0 is HMDDeviceId and is preserved. The remaining are action devices.
	if (DeviceSpaces.Num() > 0)
	{
		DeviceSpaces.RemoveAt(HMDDeviceId + 1, DeviceSpaces.Num() - 1);
	}
}

XrPath FOpenXRHMD::GetTrackedDevicePath(const int32 DeviceId)
{
	FReadScopeLock DeviceLock(DeviceMutex);
	if (DeviceSpaces.IsValidIndex(DeviceId))
	{
		return DeviceSpaces[DeviceId].Path;
	}
	return XR_NULL_PATH;
}

XrSpace FOpenXRHMD::GetTrackedDeviceSpace(const int32 DeviceId)
{
	FReadScopeLock DeviceLock(DeviceMutex);
	if (DeviceSpaces.IsValidIndex(DeviceId))
	{
		return DeviceSpaces[DeviceId].Space;
	}
	return XR_NULL_HANDLE;
}

XrTime FOpenXRHMD::GetDisplayTime() const
{
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();
	return PipelineState.bXrFrameStateUpdated ? PipelineState.FrameState.predictedDisplayTime : 0;
}

XrSpace FOpenXRHMD::GetTrackingSpace() const
{
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();
	if (PipelineState.TrackingSpace.IsValid())
	{
		return PipelineState.TrackingSpace->Handle;
	}
	else
	{
		return XR_NULL_HANDLE;
	}
}

bool FOpenXRHMD::AllocateSwapchainTextures_RenderThread(const FOpenXRSwapchainProperties& InSwapchainProperties, FXRSwapChainPtr& InOutSwapchain, uint8& OutActualFormat)
{
	check(IsInRenderingThread());

	FReadScopeLock Lock(SessionHandleMutex);
	if (!Session)
	{
		return false;
	}
	
	const FRHITexture* const SwapchainTexture = InOutSwapchain == nullptr ? nullptr : InOutSwapchain->GetTexture2DArray() ? InOutSwapchain->GetTexture2DArray() : InOutSwapchain->GetTexture2D();

	if (InOutSwapchain == nullptr ||
		SwapchainTexture == nullptr ||
		SwapchainTexture->GetFormat() != InSwapchainProperties.Format || 
		SwapchainTexture->GetSizeX() != InSwapchainProperties.SizeX ||
		SwapchainTexture->GetSizeY() != InSwapchainProperties.SizeY ||
		SwapchainTexture->GetDesc().ArraySize != InSwapchainProperties.ArraySize ||
		SwapchainTexture->GetNumMips() != InSwapchainProperties.NumMips ||
		SwapchainTexture->GetNumSamples() != InSwapchainProperties.NumSamples ||
		SwapchainTexture->GetFlags() != InSwapchainProperties.CreateFlags)
	{
		InOutSwapchain = RenderBridge->CreateSwapchain(
			Session,
			InSwapchainProperties.Format,
			OutActualFormat,
			InSwapchainProperties.SizeX,
			InSwapchainProperties.SizeY,
			InSwapchainProperties.ArraySize,
			InSwapchainProperties.NumMips,
			InSwapchainProperties.NumSamples,
			InSwapchainProperties.Dimension,
			InSwapchainProperties.CreateFlags,
			InSwapchainProperties.ClearValueBinding,
			InSwapchainProperties.AuxiliaryCreateFlags);
		
		if (InOutSwapchain)
		{
			UE_LOGF(LogHMD, Verbose, "Allocated %.*ls OpenXR Swapchain of Format: %u, SizeX: %u, SizeY: %u, ArraySize: %u, NumMips: %u, NumSamples: %u, Flags: %llu",
				InSwapchainProperties.DebugName.Len(),
				InSwapchainProperties.DebugName.GetData(),
				InSwapchainProperties.Format,
				InSwapchainProperties.SizeX,
				InSwapchainProperties.SizeY,
				InSwapchainProperties.ArraySize,
				InSwapchainProperties.NumMips,
				InSwapchainProperties.NumSamples,
				EnumToUnderlyingType(InSwapchainProperties.CreateFlags));
			
        	InOutSwapchain->SetDebugLabel(InSwapchainProperties.DebugName);
			return true;
        }
	}

	return false;
}

bool FOpenXRHMD::IsInitialized() const
{
	return Instance != XR_NULL_HANDLE;
}

bool FOpenXRHMD::IsRunning() const
{
	return bIsRunning;
}

bool FOpenXRHMD::IsFocused() const
{
	return CurrentSessionState == XR_SESSION_STATE_FOCUSED;
}

void FOpenXRHMD::SetEnvironmentBlendMode(XrEnvironmentBlendMode NewBlendMode) 
{
	if (NewBlendMode == XR_ENVIRONMENT_BLEND_MODE_MAX_ENUM) 
	{
		UE_LOGF(LogHMD, Error, "Environment Blend Mode can't be set to XR_ENVIRONMENT_BLEND_MODE_MAX_ENUM.");
		return;
	}

	if(!Instance || !System)
	{
		return;
	}

	TArray<XrEnvironmentBlendMode> BlendModes = GetSupportedEnvironmentBlendModes();

	if (BlendModes.Contains(NewBlendMode))
	{
		SelectedEnvironmentBlendMode = NewBlendMode;
		UE_LOGF(LogHMD, Log, "Environment Blend Mode set to: %d.", SelectedEnvironmentBlendMode);
	}
	else
	{
		UE_LOGF(LogHMD, Error, "Environment Blend Mode %d is not supported. Environment Blend Mode remains %d.", NewBlendMode, SelectedEnvironmentBlendMode);
	}
}

bool FOpenXRHMD::StartSession()
{
	// If the session is not yet ready, we'll call into this function again when it is
	FWriteScopeLock Lock(SessionHandleMutex);
	if (!bIsReady || bIsRunning)
	{
		return false;
	}

	check(InputModule);
	InputModule->OnBeginSession();

	XrSessionBeginInfo Begin = { XR_TYPE_SESSION_BEGIN_INFO, nullptr, SelectedViewConfigurationType };
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Begin.next = Module->OnBeginSession(Session, Begin.next);
	}

	bIsRunning = XR_ENSURE(xrBeginSession(Session, &Begin));
	return bIsRunning;
}

bool FOpenXRHMD::StopSession()
{
	FWriteScopeLock Lock(SessionHandleMutex);
	if (!bIsRunning)
	{
		return false;
	}

	bIsRunning = !XR_ENSURE(xrEndSession(Session));
	return !bIsRunning;
}

void FOpenXRHMD::OnBeginPlay(FWorldContext& InWorldContext)
{
	bOpenXRForceStereoLayersEmulationCVarCachedValue = CVarOpenXRForceStereoLayerEmulation.GetValueOnGameThread();
}

IStereoRenderTargetManager* FOpenXRHMD::GetRenderTargetManager()
{
	return this;
}

int32 FOpenXRHMD::AcquireColorTexture()
{
	check(IsInGameThread());
	if (Session)
	{
		const FXRSwapChainPtr& ColorSwapchain = PipelinedLayerStateRendering.ColorSwapchain;
		if (ColorSwapchain)
		{
			if (bIsAcquireOnAnyThreadSupported)
			{
				ColorSwapchain->IncrementSwapChainIndex_RHIThread();
			}
			return ColorSwapchain->GetSwapChainIndex_RHIThread();
		}
	}
	return 0;
}

int32 FOpenXRHMD::AcquireDepthTexture()
{
	check(IsInGameThread());
	if (Session)
	{
		const FXRSwapChainPtr& DepthSwapchain = PipelinedLayerStateRendering.DepthSwapchain;
		const FXRSwapChainPtr& MotionVectorSwapchain = PipelinedLayerStateRendering.MotionVectorSwapchain;
		const FXRSwapChainPtr& MotionVectorDepthSwapchain = PipelinedLayerStateRendering.MotionVectorDepthSwapchain;

		// In the future, this function should be merged with AcquireColorTexture() into a single function that acquires all textures
		// We currently acquire all textures at the same time in FSceneViewport::EnqueueBeginRenderFrame and only use the index returned by AcquireColorTexture()
		if (MotionVectorSwapchain && MotionVectorDepthSwapchain)
		{
			if (bIsAcquireOnAnyThreadSupported)
			{
				MotionVectorSwapchain->IncrementSwapChainIndex_RHIThread();
				MotionVectorDepthSwapchain->IncrementSwapChainIndex_RHIThread();
			}
		}

		if (DepthSwapchain)
		{
			if (bIsAcquireOnAnyThreadSupported)
			{
				DepthSwapchain->IncrementSwapChainIndex_RHIThread();
			}
			return DepthSwapchain->GetSwapChainIndex_RHIThread();
		}
	}
	return 0;
}

bool FOpenXRHMD::AllocateRenderTargetTextures(FRHICommandListBase& RHICmdList, uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ETextureCreateFlags TargetableTextureFlags, TArray<FTextureRHIRef>& OutTargetableTextures, TArray<FTextureRHIRef>& OutShaderResourceTextures, uint32 NumSamples)
{
	check(IsInRenderingThread());

	FReadScopeLock Lock(SessionHandleMutex);
	if (!Session)
	{
		return false;
	}

	// We're only creating a 1x target here, but we don't know whether it'll be the targeted texture
	// or the resolve texture. Because of this, we unify the input flags.
	ETextureCreateFlags UnifiedCreateFlags = Flags | TargetableTextureFlags;

	// This is not a static swapchain
	UnifiedCreateFlags |= TexCreate_Dynamic;

	// We need to ensure we can sample from the texture in CopyTexture
	UnifiedCreateFlags |= TexCreate_ShaderResource;

	// We assume this could be used as a resolve target
	UnifiedCreateFlags |= TexCreate_ResolveTargetable;

	// Some render APIs require us to present in RT layouts/configs,
	// so even if app won't use this texture as RT, we need the flag.
	UnifiedCreateFlags |= TexCreate_RenderTargetable;

	// On mobile without HDR all render targets need to be marked sRGB
	bool MobileHWsRGB = IsMobileColorsRGB() && IsMobilePlatform(GetConfiguredShaderPlatform());
	if (MobileHWsRGB)
	{
		UnifiedCreateFlags |= TexCreate_SRGB;
	}
	ETextureCreateFlags AuxiliaryCreateFlags = ETextureCreateFlags::None;

	if(FBFoveationImageGenerator && FBFoveationImageGenerator->IsFoveationExtensionEnabled())
	{
		AuxiliaryCreateFlags |= TexCreate_Foveation;
	}

	// Allow extension plugins to override the color swapchain format.
	uint8 PluginFormat = PF_Unknown;
	FString PluginFormatExtensionName;
	for (IOpenXRExtensionPlugin* Plugin : ExtensionPlugins)
	{
		const uint8 CurrentPluginFormat = Plugin->GetPreferredSwapchainFormat(Format);

		if (CurrentPluginFormat != PF_Unknown && CurrentPluginFormat < PF_MAX)
		{
			if (PluginFormat == PF_Unknown)
			{
				PluginFormat = CurrentPluginFormat;
				PluginFormatExtensionName = Plugin->GetDisplayName();
			}
			else if (CurrentPluginFormat != PluginFormat)
			{
				UE_LOG(LogHMD, Warning, TEXT("Extension %s requested swapchain format %s via GetPreferredSwapchainFormat, but we are already using format %s requested by extension %s."), 
					*Plugin->GetDisplayName(),
					GetPixelFormatString(static_cast<EPixelFormat>(CurrentPluginFormat)),
					GetPixelFormatString(static_cast<EPixelFormat>(PluginFormat)),
					*PluginFormatExtensionName);
			}
		}

		else if (CurrentPluginFormat >= PF_MAX)
		{
			UE_LOG(LogHMD, Warning, TEXT("Extension %s requested invalid swapchain format %d via GetPreferredSwapchainFormat."), *Plugin->GetDisplayName(), CurrentPluginFormat);
		}
	}

	if (PluginFormat != PF_Unknown)
	{
		Format = PluginFormat;
	}

	// Temporary workaround to swapchain formats - OpenXR doesn't support 10-bit sRGB swapchains, so prefer 8-bit sRGB instead.
	if (Format == PF_A2B10G10R10 && !RenderBridge->Support10BitSwapchain())
	{
		UE_LOGF(LogHMD, Warning, "Requesting 10 bit swapchain, but not supported: fall back to 8bpc.");
		// Match the default logic in GetDefaultMobileSceneColorLowPrecisionFormat() in SceneTexturesConfig.cpp
		Format = IsStandaloneStereoOnlyDevice() ? PF_R8G8B8A8 : PF_B8G8R8A8;
	}

	FXRSwapChainPtr& Swapchain = PipelinedLayerStateRendering.ColorSwapchain;
	/*if (!Swapchain.IsValid())
	{
		return false;
	}*/

	uint8 ActualFormat = Format;
	ensureMsgf(NumSamples == 1, TEXT("OpenXR supports MSAA swapchains, but engine logic expects the swapchain target to be 1x."));
	
	const FOpenXRSwapchainProperties ColorSwapchainProperties = {
		TEXT("ColorSwapchain"),
		Format,
		bIsMobileMultiViewEnabled ? ETextureDimension::Texture2DArray : ETextureDimension::Texture2D,
		SizeX,
		SizeY,
		static_cast<uint32>(bIsMobileMultiViewEnabled ? 2 : 1),
		NumMips,
		NumSamples,
		UnifiedCreateFlags,
		FClearValueBinding::Transparent,
		AuxiliaryCreateFlags
	};

	bool bAllocatedColorTextures = AllocateSwapchainTextures_RenderThread(ColorSwapchainProperties, Swapchain, ActualFormat);

	// Image will be acquired by the renderer if supported, if not we acquire it ahead of time here
	if (!bIsAcquireOnAnyThreadSupported)
	{
		RHICmdList.EnqueueLambda([this, Swapchain](FRHICommandListBase&)
		{
			RenderBridge->RunOnRHISubmissionThread([Swapchain]()
			{
				Swapchain->IncrementSwapChainIndex_RHIThread();
			});
		});
		RHICmdList.GetAsImmediate().ImmediateFlush(EImmediateFlushType::FlushRHIThread);
	}

	if (FBFoveationImageGenerator && FBFoveationImageGenerator->IsFoveationExtensionEnabled())
	{
		FBFoveationImageGenerator->UpdateFoveationImages(/* bReallocatedSwapchain */ bAllocatedColorTextures);
	}

	// Grab the presentation texture out of the swapchain.
	OutTargetableTextures = Swapchain->GetSwapChain();
	OutShaderResourceTextures = OutTargetableTextures;
	LastRequestedColorSwapchainFormat = Format;
	LastActualColorSwapchainFormat = ActualFormat;

	if (IsEmulatingStereoLayers() && (SystemProperties.graphicsProperties.maxLayerCount > 1))
	{
		// If we have at least two native layers, use non-background layer to render the composited image of all the emulated face locked layers.
		const ETextureCreateFlags EmulationCreateFlags = TexCreate_Dynamic | TexCreate_ShaderResource | TexCreate_RenderTargetable;
		uint8 UnusedActualFormat = 0;

		const FOpenXRSwapchainProperties EmulationSwapchainProperties = {
			TEXT("EmulationSwapchain"),
			IStereoRenderTargetManager::GetStereoLayerPixelFormat(),
			bIsMobileMultiViewEnabled ? ETextureDimension::Texture2DArray : ETextureDimension::Texture2D,
			SizeX,
			SizeY,
			static_cast<uint32>(bIsMobileMultiViewEnabled ? 2 : 1),
			NumMips,
			NumSamples,
			EmulationCreateFlags,
			FClearValueBinding::Transparent
		};

		AllocateSwapchainTextures_RenderThread(EmulationSwapchainProperties, PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain, UnusedActualFormat);

		// Image will be acquired by the renderer if supported, if not we acquire it ahead of time here
		FXRSwapChainPtr& EmulationSwapchain = PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain;
		if (!bIsAcquireOnAnyThreadSupported && EmulationSwapchain.IsValid())
		{
			RHICmdList.EnqueueLambda([this, EmulationSwapchain](FRHICommandListBase&)
			{
				RenderBridge->RunOnRHISubmissionThread([EmulationSwapchain]
				{
					EmulationSwapchain->IncrementSwapChainIndex_RHIThread();
				});
			});
			RHICmdList.GetAsImmediate().ImmediateFlush(EImmediateFlushType::FlushRHIThread);
		}
	}

	// Only create motion vector + motion vector depth swapchains if XR_EXT_frame_synthesis or XR_FB_space_warp are supported and r.Velocity.DirectlyRenderOpenXRMotionVectors=1
	FIntPoint MotionVectorSwapchainSize = FIntPoint();
	if (GetRecommendedMotionVectorTextureSize(MotionVectorSwapchainSize) && PlatformSupportsOpenXRMotionVectors(GetConfiguredShaderPlatform()))
	{
		const ETextureCreateFlags MotionVectorCreateFlags = TexCreate_RenderTargetable | TexCreate_ResolveTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead | TexCreate_Dynamic;
		uint8 UnusedActualFormat = 0;

		const FOpenXRSwapchainProperties MotionVectorSwapchainProperties = {
			TEXT("MotionVectorSwapchain"),
			PF_FloatRGBA,
			bIsMobileMultiViewEnabled ? ETextureDimension::Texture2DArray : ETextureDimension::Texture2D,
			static_cast<uint32>(MotionVectorSwapchainSize.X),
			static_cast<uint32>(MotionVectorSwapchainSize.Y),
			static_cast<uint32>(bIsMobileMultiViewEnabled ? 2 : 1),
			1,
			1,
			MotionVectorCreateFlags,
			FClearValueBinding::Transparent
		};

		AllocateSwapchainTextures_RenderThread(MotionVectorSwapchainProperties, PipelinedLayerStateRendering.MotionVectorSwapchain, UnusedActualFormat);

		const FOpenXRSwapchainProperties MotionVectorDepthSwapchainProperties = {
			TEXT("MotionVectorDepthSwapchain"),
			PF_DepthStencil,
			bIsMobileMultiViewEnabled ? ETextureDimension::Texture2DArray : ETextureDimension::Texture2D,
			static_cast<uint32>(MotionVectorSwapchainSize.X),
			static_cast<uint32>(MotionVectorSwapchainSize.Y),
			static_cast<uint32>(bIsMobileMultiViewEnabled ? 2 : 1),
			1,
			1,
			TexCreate_DepthStencilTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead | TexCreate_Dynamic,
			FClearValueBinding::DepthZero,
			TexCreate_None
		};
		
		AllocateSwapchainTextures_RenderThread(MotionVectorDepthSwapchainProperties, PipelinedLayerStateRendering.MotionVectorDepthSwapchain, UnusedActualFormat);

		// Images will be acquired by the renderer if supported, if not we acquire it ahead of time here
		FXRSwapChainPtr& MotionVectorSwapchain = PipelinedLayerStateRendering.MotionVectorSwapchain;
		FXRSwapChainPtr& MotionVectorDepthSwapchain = PipelinedLayerStateRendering.MotionVectorDepthSwapchain;
		if (!bIsAcquireOnAnyThreadSupported && MotionVectorSwapchain.IsValid() && MotionVectorDepthSwapchain.IsValid())
		{
			RHICmdList.EnqueueLambda([this, MotionVectorSwapchain, MotionVectorDepthSwapchain](FRHICommandListBase&)
			{
				RenderBridge->RunOnRHISubmissionThread([MotionVectorSwapchain, MotionVectorDepthSwapchain]
				{
					MotionVectorSwapchain->IncrementSwapChainIndex_RHIThread();
					MotionVectorDepthSwapchain->IncrementSwapChainIndex_RHIThread();
				});
			});
			RHICmdList.GetAsImmediate().ImmediateFlush(EImmediateFlushType::FlushRHIThread);
		}
	}

	// TODO: Pass in known depth parameters (format + flags)? Do we know that at viewport setup time?
	AllocateDepthTextureInternal(SizeX, SizeY, NumSamples, bIsMobileMultiViewEnabled ? 2 : 1);

	return true;
}

void FOpenXRHMD::AllocateDepthTextureInternal(uint32 SizeX, uint32 SizeY, uint32 NumSamples, uint32 InArraySize)
{
	check(IsInRenderingThread());

	FReadScopeLock Lock(SessionHandleMutex);
	if (!Session || !bDepthExtensionSupported)
	{
		return;
	}

	// We're only creating a 1x target here, but we don't know whether it'll be the targeted texture
	// or the resolve texture. Because of this, we unify the input flags.
	ETextureCreateFlags UnifiedCreateFlags = TexCreate_DepthStencilTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead;

	// This is not a static swapchain
	UnifiedCreateFlags |= TexCreate_Dynamic;

	// We assume this could be used as a resolve target
	UnifiedCreateFlags |= TexCreate_DepthStencilResolveTarget;

	ensureMsgf(NumSamples == 1, TEXT("OpenXR supports MSAA swapchains, but engine logic expects the swapchain target to be 1x."));
	constexpr uint32 NumSamplesExpected = 1;
	constexpr uint32 NumMipsExpected = 1;
	uint8 UnusedActualFormat = 0;

	const FOpenXRSwapchainProperties DepthSwapchainProperties = {
		TEXT("DepthSwapchain"),
		PF_DepthStencil,
		bIsMobileMultiViewEnabled ? ETextureDimension::Texture2DArray : ETextureDimension::Texture2D,
		SizeX,
		SizeY,
		InArraySize,
		NumMipsExpected,
		NumSamplesExpected,
		UnifiedCreateFlags,
		FClearValueBinding::DepthFar
	};

	AllocateSwapchainTextures_RenderThread(DepthSwapchainProperties, PipelinedLayerStateRendering.DepthSwapchain, UnusedActualFormat);

	// Image will be acquired by the renderer if supported, if not we acquire it ahead of time here
	if (!bIsAcquireOnAnyThreadSupported)
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		FXRSwapChainPtr& DepthSwapchain = PipelinedLayerStateRendering.DepthSwapchain;
		RHICmdList.EnqueueLambda([this, DepthSwapchain](FRHICommandListImmediate&)
		{
			RenderBridge->RunOnRHISubmissionThread([DepthSwapchain]()
			{
				DepthSwapchain->IncrementSwapChainIndex_RHIThread();
			});
		});
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
	}
}

// TODO: in the future, we can rename the interface to GetDepthTexture because allocate could happen in AllocateRenderTargetTexture
bool FOpenXRHMD::AllocateDepthTexture(uint32 Index, uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ETextureCreateFlags TargetableTextureFlags, FTextureRHIRef& OutTargetableTexture, FTextureRHIRef& OutShaderResourceTexture, uint32 NumSamples)
{
	check(IsInRenderingThread());

	// FIXME: UE constantly calls this function even when there is no reason to reallocate the depth texture (see NeedReAllocateDepthTexture)
	FReadScopeLock Lock(SessionHandleMutex);
	if (!Session || !bDepthExtensionSupported)
	{
		return false;
	}

	const FXRSwapChainPtr& DepthSwapchain = PipelinedLayerStateRendering.DepthSwapchain;
	if (DepthSwapchain == nullptr)
	{
		return false;
	}

	const ETextureCreateFlags UnifiedCreateFlags = Flags | TargetableTextureFlags;
	ensure(EnumHasAllFlags(UnifiedCreateFlags, TexCreate_DepthStencilTargetable)); // We can't use the depth swapchain w/o this flag
	const FRHITexture* const DepthSwapchainTexture = DepthSwapchain->GetTexture2DArray() ? DepthSwapchain->GetTexture2DArray() : DepthSwapchain->GetTexture2D();
	const FRHITextureDesc& DepthSwapchainDesc = DepthSwapchainTexture->GetDesc();

	if (SizeX != DepthSwapchainDesc.Extent.X || SizeY != DepthSwapchainDesc.Extent.Y)
	{
		// We don't yet support different sized SceneTexture depth + OpenXR layer depth
		return false;
	}

	// Sample count, mip count and size should be known at AllocateRenderTargetTexture time
	// Format _could_ change, but we should know it (and can check for it in AllocateDepthTextureInternal)
	// Flags might also change. We expect TexCreate_DepthStencilTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead from SceneTextures
	check(EnumHasAllFlags(DepthSwapchainDesc.Flags, UnifiedCreateFlags));
	check(DepthSwapchainDesc.Format == Format);
	check(DepthSwapchainDesc.NumMips == FMath::Max(NumMips, 1u));
	check(DepthSwapchainDesc.NumSamples == NumSamples);

	LastRequestedDepthSwapchainFormat = Format;

	OutTargetableTexture = OutShaderResourceTexture = (FTextureRHIRef&)PipelinedLayerStateRendering.DepthSwapchain->GetTextureRef();

	PipelinedLayerStateRendering.LayerStateFlags |= EOpenXRLayerStateFlags::SubmitDepthLayer;

	return true;
}

bool FOpenXRHMD::GetRecommendedMotionVectorTextureSize(FIntPoint& OutTextureSize)
{
	if (RecommendedMotionVectorTextureSize.X == 0 || RecommendedMotionVectorTextureSize.Y == 0)
	{
		OutTextureSize = FIntPoint();
		return false;
	}
	
	OutTextureSize = RecommendedMotionVectorTextureSize;
	return true;
}

bool FOpenXRHMD::GetMotionVectorTexture(uint32 Index, const FIntPoint& Size, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, FTextureRHIRef& OutTexture, uint32 NumSamples)
{
	check(IsInRenderingThread());

	const FXRSwapChainPtr& MotionVectorSwapchain = PipelinedLayerStateRendering.MotionVectorSwapchain;
	if (MotionVectorSwapchain == nullptr)
	{
		return false;
	}

	const FRHITexture* const MotionVectorSwapchainTexture = MotionVectorSwapchain->GetTexture2DArray() ? MotionVectorSwapchain->GetTexture2DArray() : MotionVectorSwapchain->GetTexture2D();
	const FRHITextureDesc& MotionVectorSwapchainDesc = MotionVectorSwapchainTexture->GetDesc();
		
	if (Size != MotionVectorSwapchainDesc.Extent)
	{
		// We don't yet support motion vector texture sizes other than the recommended size
		return false;
	}

	// Sample count, mip count and size should be known at AllocateRenderTargetTexture time
	check(EnumHasAllFlags(MotionVectorSwapchainDesc.Flags, Flags));
	check(MotionVectorSwapchainDesc.Format == Format);
	check(MotionVectorSwapchainDesc.NumMips == FMath::Max(NumMips, 1u));
	check(MotionVectorSwapchainDesc.NumSamples == NumSamples);
		
	OutTexture = MotionVectorSwapchain->GetTextureRef();

	PipelinedLayerStateRendering.LayerStateFlags |= EOpenXRLayerStateFlags::SubmitMotionVectorLayer;

	return true;
}

bool FOpenXRHMD::GetMotionVectorDepthTexture(uint32 Index, const FIntPoint& Size, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, FTextureRHIRef& OutTexture, uint32 NumSamples)
{
	check(IsInRenderingThread());

	const FXRSwapChainPtr& MotionVectorDepthSwapchain = PipelinedLayerStateRendering.MotionVectorDepthSwapchain;
	if (MotionVectorDepthSwapchain == nullptr)
	{
		return false;
	}

	const FRHITexture* const MotionVectorDepthSwapchainTexture = MotionVectorDepthSwapchain->GetTexture2DArray() ? MotionVectorDepthSwapchain->GetTexture2DArray() : MotionVectorDepthSwapchain->GetTexture2D();
	const FRHITextureDesc& MotionVectorDepthSwapchainDesc = MotionVectorDepthSwapchainTexture->GetDesc();

	if (Size != MotionVectorDepthSwapchainDesc.Extent)
	{
		// We don't yet support motion vector depth texture sizes other than the recommended size
		return false;
	}

	// Sample count, mip count and size should be known at AllocateRenderTargetTexture time
	check(EnumHasAllFlags(MotionVectorDepthSwapchainDesc.Flags, Flags));
	check(MotionVectorDepthSwapchainDesc.Format == Format);
	check(MotionVectorDepthSwapchainDesc.NumMips == FMath::Max(NumMips, 1u));
	check(MotionVectorDepthSwapchainDesc.NumSamples == NumSamples);

	OutTexture = MotionVectorDepthSwapchain->GetTextureRef();

	// Our currently supported extensions always expect motion vectors + motion vector depth, can't do just one
	PipelinedLayerStateRendering.LayerStateFlags |= EOpenXRLayerStateFlags::SubmitMotionVectorLayer;

	return true;
}

void FOpenXRLayer::FPerEyeTextureData::ConfigureSwapchain(XrSession Session, FOpenXRRenderBridge* RenderBridge, FTextureRHIRef InTexture, bool bInStaticSwapchain)
{
	const bool bNewTexture = Texture != InTexture;
	Texture = InTexture;
	if (Texture.IsValid())
	{
		if (!Swapchain.IsValid() ||
			Texture->GetSizeXY() != FIntPoint(SwapchainSize.X, SwapchainSize.Y) ||
			bInStaticSwapchain != bStaticSwapchain ||
			Texture->GetDesc().Dimension != SwapchainDimension ||
			(bStaticSwapchain && bNewTexture))
		{
			ETextureCreateFlags DynamicFlag = bInStaticSwapchain ? TexCreate_None : TexCreate_Dynamic;
			const bool bIsCubemap = Texture->GetDesc().Dimension == ETextureDimension::TextureCube || Texture->GetDesc().Dimension == ETextureDimension::TextureCubeArray;
			DynamicFlag |= bIsCubemap ? TexCreate_TargetArraySlicesIndependently : TexCreate_None;
			uint8 UnusedActualFormat = 0;
			
			Swapchain = RenderBridge->CreateSwapchain(
				Session,
				IStereoRenderTargetManager::GetStereoLayerPixelFormat(),
				UnusedActualFormat,
				Texture->GetSizeX(),
				Texture->GetSizeY(),
				Texture->GetDesc().Dimension == ETextureDimension::Texture3D ? 1 : Texture->GetSizeZ(),
				Texture->GetNumMips(),
				Texture->GetNumSamples(),
				Texture->GetDesc().Dimension,
				Texture->GetFlags() | DynamicFlag | TexCreate_SRGB | TexCreate_RenderTargetable,
				Texture->GetClearBinding());
			
			SwapchainSize = FVector2D(Texture->GetSizeXY());
			SwapchainDimension = Texture->GetDesc().Dimension;
			bStaticSwapchain = bInStaticSwapchain;
			bUpdateTexture = true;
		}
	}
	else
	{
		Swapchain.Reset();
	}
}

bool FOpenXRHMD::IsEmulatingStereoLayers()
{
	return !bLayerSupportOpenXRCompliant || bOpenXRForceStereoLayersEmulationCVarCachedValue;
}

struct FLayerToUpdateSwapchain
{
	IStereoLayers::FLayerDesc Desc;
	FTextureResource* TextureResource;
	FTextureResource* LeftTextureResource;

	FLayerToUpdateSwapchain(const IStereoLayers::FLayerDesc& InDesc)
		: Desc(InDesc)
		, TextureResource(InDesc.TextureObj.IsValid() ? InDesc.TextureObj->GetResource() : nullptr)
		, LeftTextureResource(InDesc.LeftTextureObj.IsValid() ? InDesc.LeftTextureObj->GetResource() : nullptr)
	{
	}
};

void FOpenXRHMD::SetupFrameLayers_GameThread()
{
	if (GetStereoLayersDirty())
	{
		VisibleLayerIds.Reset();
		TArray<FLayerToUpdateSwapchain> SwapchainUpdates;
		TArray<FDefaultStereoLayers::FStereoLayerToRenderTransfer> LayersToRender;

		// Go over the dirtied layers to bin them into either native or emulated
		ForEachLayer([&](uint32 LayerId, FLayerDesc& Desc)
		{
			if (!(Desc.Flags & LAYER_FLAG_HIDDEN))
			{
				VisibleLayerIds.Add(LayerId);

				if (Desc.HasValidTexture())
				{
					if (IsEmulatingStereoLayers())
					{
						// Only quad layers are supported by emulation.
						if (Desc.HasShape<FQuadLayer>())
						{
							LayersToRender.Add(Desc);
#if WITH_EDITOR
							LayersToRender.Last().Name = Desc.Name;
#endif
						}
						else
						{
							UE_LOGF(LogHMD, Warning, "Layer type not supported!");
						}
					} // OpenXR compliant layer support (16 layers).
					else
					{
						if (Desc.HasShape<FQuadLayer>() || 
							(Desc.HasShape<FCylinderLayer>() && bCylinderLayersSupported) ||
							(Desc.HasShape<FEquirectLayer>() && bEquirectLayersSupported) ||
							(Desc.HasShape<FCubemapLayer>() && bCubemapLayersSupported))
						{
							SwapchainUpdates.Emplace(Desc);
						}
						else
						{
							if (Desc.HasShape<FCylinderLayer>() && !bCylinderLayersSupported)
							{
								UE_LOGF(LogHMD, Warning, "Cylinder layers not supported!");
							}
							else if (Desc.HasShape<FEquirectLayer>() && !bEquirectLayersSupported)
							{
								UE_LOGF(LogHMD, Warning, "Equirect layers not supported!");
							}
							else if (Desc.HasShape<FCubemapLayer>() && !bCubemapLayersSupported)
							{
								UE_LOGF(LogHMD, Warning, "Cubemap layers not supported!");
							}
						}
					}
				}
			}
		});

		const bool bUseLegacyFaceLockedLayersSorting = CVarOpenXRSortFaceLockedLayersAboveOtherLayers.GetValueOnAnyThread();

		// Sort from lowest priority to highest priority. Layers with higher priority will render on top of layers with lower priority.
		auto LayerCompare =
			[bUseLegacyFaceLockedLayersSorting]
			(const auto& DescA, const auto& DescB) -> bool
			{
				if (bUseLegacyFaceLockedLayersSorting && (DescA.PositionType == IStereoLayers::FaceLocked) != (DescB.PositionType ==
					IStereoLayers::FaceLocked))
				{
					return DescB.PositionType == IStereoLayers::FaceLocked;
				}
				else if (DescA.Priority != DescB.Priority)
				{
					return DescA.Priority < DescB.Priority;
				}
				else
				{
					return DescA.Id < DescB.Id;
				}
			};

		LayersToRender.Sort(LayerCompare);

		VisibleLayerIds.Sort([this, LayerCompare](const uint32& A, const uint32& B)
		{
			return LayerCompare(*FindLayerDesc(A), *FindLayerDesc(B));
		});

		SwapchainUpdates.Sort([LayerCompare](const FLayerToUpdateSwapchain& A, const FLayerToUpdateSwapchain& B)
		{
			return LayerCompare(A.Desc, B.Desc);
		});

		ENQUEUE_RENDER_COMMAND(OpenXRHMD_SetupFrameLayers)(
			[this,
				SwapchainUpdates = MoveTemp(SwapchainUpdates),
				LayersToRender = MoveTemp(LayersToRender),
				XVisibleLayerIds = VisibleLayerIds]
			(FRHICommandListImmediate& RHICmdList) mutable
			{
				TArray<FOpenXRLayer> NativeLayersBackup = MoveTemp(NativeLayers);
				NativeLayers.Reset(SwapchainUpdates.Num());

				for (const FLayerToUpdateSwapchain& Update : SwapchainUpdates)
				{
					FOpenXRLayer& Layer = NativeLayers.Emplace_GetRef(Update.Desc);
					int32 OldLayerIndex = NativeLayersBackup.IndexOfByPredicate(
						[LayerId = Layer.Desc.Id](const FOpenXRLayer& Layer)
						{
							if (Layer.Desc.Id == LayerId)
							{
								return true;
							}
							return false;
						});
					
					if (OldLayerIndex != INDEX_NONE)
					{
						Layer.RightEye = MoveTemp(NativeLayersBackup[OldLayerIndex].RightEye);
						Layer.LeftEye = MoveTemp(NativeLayersBackup[OldLayerIndex].LeftEye);
						NativeLayersBackup.RemoveAtSwap(OldLayerIndex);
					}

					ConfigureLayerSwapchains(Update, Layer);
				}

				BackgroundCompositedEmulatedLayers.Reset();
				EmulatedFaceLockedLayers.Reset();
				EmulatedLayers.Reset();

				if (UE::OpenXR::bRenderEmulatedLayerDepth)
				{
					EmulatedLayers.Append(MoveTemp(LayersToRender));
					Algo::Sort(EmulatedLayers);
				}
				else
				{
					for (FDefaultStereoLayers::FStereoLayerToRenderTransfer& Layer : LayersToRender)
					{
						if (Layer.PositionType == ELayerType::FaceLocked)
						{
							// If we have at least one native layer, use it to render the 
							// composited image of all the emulated face locked layers.
							if (PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain.IsValid())
							{
								EmulatedFaceLockedLayers.Emplace(MoveTemp(Layer));
							}
							else
							{
								BackgroundCompositedEmulatedLayers.Emplace(MoveTemp(Layer));
							}
						}
						else // Layer is not face locked
						{
							BackgroundCompositedEmulatedLayers.Emplace(MoveTemp(Layer));
						}
					}
				}
			});
	} //GetStereoLayersDirty()
}

void FOpenXRHMD::MarkTextureForUpdate(uint32 LayerId)
{
	const FLayerDesc* Desc = FindLayerDesc(LayerId);
	if (!Desc)
	{
		return; // Layer has been deleted
	}

	FLayerToUpdateSwapchain Update(*Desc);
	ENQUEUE_RENDER_COMMAND(UpdateLayerTexture)(
		[
			this,
			Update=MoveTemp(Update)
		](FRHICommandList&)
	{
		for (FOpenXRLayer& NativeLayer : NativeLayers)
		{
			if (NativeLayer.Desc.Id == Update.Desc.Id)
			{
				NativeLayer.Desc = Update.Desc;
				NativeLayer.RightEye.bUpdateTexture = true;
				NativeLayer.LeftEye.bUpdateTexture = true;
				ConfigureLayerSwapchains(Update, NativeLayer);

				break;
			}
		}
		// If we don't find the layer, that's fine. It will be added and updated next frame if needed.
	});
}

void FOpenXRHMD::ConfigureLayerSwapchains(const FLayerToUpdateSwapchain& Update, FOpenXRLayer& Layer)
{
	auto GetTexture =
		[](FTextureResource* Resource, ETextureDimension Dimension) -> FTextureRHIRef
		{
			FTextureRHIRef Tex = Resource ? Resource->GetTextureRHI() : nullptr;
			if (Tex && Tex->GetDesc().Dimension != Dimension)
			{
				UE_LOGF(LogHMD, Error, "Texture dimensions do not match");
				return nullptr;
			}
			return Tex;
		};

	ETextureDimension TargetDimension = Layer.Desc.HasShape<FCubemapLayer>() ? ETextureDimension::TextureCube : ETextureDimension::Texture2D;
	FTextureRHIRef Texture = GetTexture(Update.TextureResource, TargetDimension);
	FTextureRHIRef LeftTexture = GetTexture(Update.LeftTextureResource, TargetDimension);
	bool bStaticSwapchain = !(Layer.Desc.Flags & LAYER_FLAG_TEX_CONTINUOUS_UPDATE);
	Layer.RightEye.ConfigureSwapchain(Session, RenderBridge, Texture, bStaticSwapchain);
	Layer.LeftEye.ConfigureSwapchain(Session, RenderBridge, LeftTexture, bStaticSwapchain);
}

void FOpenXRHMD::SetupFrameLayers_RenderThread(FRDGBuilder& GraphBuilder)
{
	ensure(IsInRenderingThread());

	PipelinedLayerStateRendering.LayerStateFlags |= !EmulatedFaceLockedLayers.IsEmpty() ? EOpenXRLayerStateFlags::SubmitEmulatedFaceLockedLayer : EOpenXRLayerStateFlags::None;

	if (bIsAcquireOnAnyThreadSupported && PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain)
	{
		PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain->IncrementSwapChainIndex_RHIThread();
	}

	const FTransform InvTrackingToWorld = GetTrackingToWorldTransform().Inverse();
	const float WorldToMeters = GetWorldToMetersScale();

	const int32 NumHeaders = GetNumHeaders();
	PipelinedLayerStateRendering.NativeOverlays.Reset(NumHeaders);
	PipelinedLayerStateRendering.NativeUnderlays.Reset(NumHeaders);
	PipelinedLayerStateRendering.NativeLayerSwapchains.Reset(NumHeaders);
	PipelinedLayerStateRendering.CompositionDepthTestLayers.Reset(NumHeaders);

	const bool bUseLegacyFaceLockedLayersSorting = CVarOpenXRSortFaceLockedLayersAboveOtherLayers.GetValueOnAnyThread();
	const int32 ProjectionLayerPriority = CVarOpenXRProjectionLayerPriority.GetValueOnAnyThread();

	// Set up our OpenXR info per native layer. Emulated layers have everything in FLayerDesc.
	for (FOpenXRLayer& Layer : NativeLayers)
	{
		FReadScopeLock DeviceLock(DeviceMutex);

		const bool bIsFaceLocked = Layer.Desc.PositionType == ELayerType::FaceLocked;
		XrSpace Space = bIsFaceLocked ? DeviceSpaces[HMDDeviceId].Space : PipelinedFrameStateRendering.TrackingSpace->Handle;
		
		EOpenXRLayerCreationFlags LayerCreationFlags = EOpenXRLayerCreationFlags::None;
		LayerCreationFlags |= IsExtensionEnabled(XR_KHR_COMPOSITION_LAYER_EQUIRECT2_EXTENSION_NAME) ? EOpenXRLayerCreationFlags::EquirectLayer2Supported : EOpenXRLayerCreationFlags::None;
		LayerCreationFlags |= IsExtensionEnabled(XR_FB_COMPOSITION_LAYER_DEPTH_TEST_EXTENSION_NAME) ? EOpenXRLayerCreationFlags::DepthTestSupported : EOpenXRLayerCreationFlags::None;

		TArray<FXrCompositionLayerUnion> Headers = Layer.CreateOpenXRLayer(InvTrackingToWorld, WorldToMeters, Space, LayerCreationFlags);
		Layer.ApplyCompositionDepthTestLayer(Headers, LayerCreationFlags, PipelinedLayerStateRendering.CompositionDepthTestLayers);

		if (Layer.Desc.Priority >= ProjectionLayerPriority || (bUseLegacyFaceLockedLayersSorting && bIsFaceLocked))
		{
			PipelinedLayerStateRendering.NativeOverlays.Append(Headers);
		}
		else
		{
			PipelinedLayerStateRendering.NativeUnderlays.Append(Headers);
		}
		
		UpdateLayerSwapchainTexture(Layer, GraphBuilder);
	}
}

void FOpenXRHMD::UpdateLayerSwapchainTexture(FOpenXRLayer& Layer, FRDGBuilder& GraphBuilder)
{
	const bool bNoAlpha = Layer.Desc.Flags & IStereoLayers::LAYER_FLAG_TEX_NO_ALPHA_CHANNEL;
	const EXRCopyTextureBlendModifier SrcTextureCopyModifier = bNoAlpha ? EXRCopyTextureBlendModifier::Opaque : EXRCopyTextureBlendModifier::TransparentAlphaPassthrough;

	FStaticFeatureLevel FeatureLevel = GMaxRHIFeatureLevel; // TODO get configured preview shader platform
	FStaticShaderPlatform ShaderPlatform = GetConfiguredShaderPlatform();

	// We need to copy each layer into an OpenXR swapchain so they can be displayed by the compositor.
	if (Layer.RightEye.Swapchain.IsValid() && Layer.RightEye.Texture.IsValid())
	{
		if (Layer.RightEye.bUpdateTexture && bIsRunning)
		{
			FRHITexture* SrcTexture = Layer.RightEye.Texture;
			FIntRect DstRect(FIntPoint(0, 0), Layer.RightEye.SwapchainSize.IntPoint());
			FRDGTextureRef SrcTextureRDG = RegisterExternalTexture(GraphBuilder, SrcTexture, TEXT("OpenXR_Layer_Texture"));
			CopySwapchainTexture_RenderThread(GraphBuilder, SrcTextureRDG, FIntRect(), Layer.RightEye.Swapchain, DstRect, false, SrcTextureCopyModifier, FeatureLevel, ShaderPlatform);
			Layer.RightEye.bUpdateTexture = !Layer.RightEye.bStaticSwapchain;
		}
		PipelinedLayerStateRendering.NativeLayerSwapchains.Add(Layer.RightEye.Swapchain);
	}
	if (Layer.LeftEye.Swapchain.IsValid() && Layer.LeftEye.Texture.IsValid())
	{
		if (Layer.LeftEye.bUpdateTexture && bIsRunning)
		{
			FRHITexture* SrcTexture = Layer.LeftEye.Texture;
			FIntRect DstRect(FIntPoint(0, 0), Layer.LeftEye.SwapchainSize.IntPoint());
			FRDGTextureRef SrcTextureRDG = RegisterExternalTexture(GraphBuilder, SrcTexture, TEXT("OpenXR_Layer_LeftTexture"));
			CopySwapchainTexture_RenderThread(GraphBuilder, SrcTextureRDG, FIntRect(), Layer.LeftEye.Swapchain, DstRect, false, SrcTextureCopyModifier, FeatureLevel, ShaderPlatform);
			Layer.LeftEye.bUpdateTexture = !Layer.LeftEye.bStaticSwapchain;
		}
		PipelinedLayerStateRendering.NativeLayerSwapchains.Add(Layer.LeftEye.Swapchain);
	}
}

void FOpenXRHMD::DrawEmulatedLayers_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView)
{	
	check(IsInRenderingThread());

	if (!IsEmulatingStereoLayers() || !IStereoRendering::IsStereoEyeView(InView))
	{
		return;
	}

	if (UE::OpenXR::bRenderEmulatedLayerDepth && PipelinedLayerStateRendering.DepthSwapchain)
	{
		DrawAllEmulatedLayers_RenderThread(GraphBuilder, InView);
	}
	else
	{
		DrawBackgroundCompositedEmulatedLayers_RenderThread(GraphBuilder, InView);
		DrawEmulatedFaceLockedLayers_RenderThread(GraphBuilder, InView);
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FEmulatedLayersPass, )
	RDG_TEXTURE_ACCESS_ARRAY(LayerTextures)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FOpenXRHMD::DrawEmulatedFaceLockedLayers_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView)
{
	if (!EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitEmulatedFaceLockedLayer))
	{
		return;
	}

	FXRSwapChainPtr EmulationSwapchain = PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain;
	FTextureRHIRef RenderTarget = EmulationSwapchain->GetTextureRef();
	FRDGTextureRef RDGRenderTarget = RegisterExternalTexture(GraphBuilder, RenderTarget, TEXT("OpenXR_EmulationSwapchain"));

	FDefaultStereoLayers_LayerRenderParams RenderParams;
	FEmulatedLayersPass* PassInfo = SetupEmulatedLayersRenderPass(GraphBuilder, InView, EmulatedFaceLockedLayers, RDGRenderTarget, RenderParams);
	
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("OpenXREmulatedFaceLockedLayerRender"),
		PassInfo,
		ERDGPassFlags::Raster,
		[this, RenderParams](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.SetViewport(
				(float)RenderParams.Viewport.Min.X,
				(float)RenderParams.Viewport.Min.Y,
				0.0f,
				(float)RenderParams.Viewport.Max.X,
				(float)RenderParams.Viewport.Max.Y,
				1.0f);

			// We need to clear to black + 0 alpha in order to composite opaque + transparent layers correctly
			DrawClearQuad(RHICmdList, FLinearColor::Transparent);

			FDefaultStereoLayers::StereoLayerRender(RHICmdList, EmulatedFaceLockedLayers, RenderParams);
		});
}

void FOpenXRHMD::DrawAllEmulatedLayers_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView)
{
	FRDGTextureRef RDGDepthTexture = RegisterExternalTexture(
		GraphBuilder,
		PipelinedLayerStateRendering.DepthSwapchain->GetTexture(),
		TEXT("OpenXR_Depth"));
	
	FTextureRHIRef RenderTarget = InView.Family->RenderTarget->GetRenderTargetTexture();
	FRDGTextureRef RDGRenderTarget = RegisterExternalTexture(GraphBuilder, RenderTarget, TEXT("ViewFamilyTexture"));
	
	auto RenderLayer =
		[this, RDGDepthTexture, RDGRenderTarget, &GraphBuilder, &InView](const FDefaultStereoLayers::FStereoLayerToRender& Layer)
		{
			FDefaultStereoLayers_LayerRenderParams RenderParams;
			FEmulatedLayersPass* PassInfo = SetupEmulatedLayerRenderPass(GraphBuilder, InView, Layer, RDGRenderTarget, RDGDepthTexture, RenderParams);
		
			GraphBuilder.AddPass(
#if WITH_EDITOR
			RDG_EVENT_NAME("OpenXREmulatedStereoLayerRender_%s", *Layer.Name),
#else
			RDG_EVENT_NAME("OpenXREmulatedStereoLayerRender"),
#endif
			PassInfo,
			ERDGPassFlags::Raster,
			[this, RenderParams, Layer](FRHICommandListImmediate& RHICmdList)
			{
				FDefaultStereoLayers::StereoLayerRender(RHICmdList, Layer, RenderParams);
			});
		};
	
	TArray<const FDefaultStereoLayers::FStereoLayerToRender*> NonWorldLocked;
	
	for (const FDefaultStereoLayers::FStereoLayerToRender& Layer : EmulatedLayers)
	{
		if (Layer.PositionType != ELayerType::WorldLocked)
		{
			NonWorldLocked.Add(&Layer);
			continue;
		}
		
		RenderLayer(Layer);
	}
	
	for (const FDefaultStereoLayers::FStereoLayerToRender* Layer : NonWorldLocked)
	{
		RenderLayer(*Layer);
	}
}

void FOpenXRHMD::DrawBackgroundCompositedEmulatedLayers_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView)
{
	FTextureRHIRef RenderTarget = InView.Family->RenderTarget->GetRenderTargetTexture();
	FRDGTextureRef RDGRenderTarget = RegisterExternalTexture(GraphBuilder, RenderTarget, TEXT("ViewFamilyTexture"));

	FDefaultStereoLayers_LayerRenderParams RenderParams;
	FEmulatedLayersPass* PassInfo = SetupEmulatedLayersRenderPass(GraphBuilder, InView, BackgroundCompositedEmulatedLayers, RDGRenderTarget, RenderParams);
	
	// Partially borrowed from FDefaultStereoLayers
	GraphBuilder.AddPass(RDG_EVENT_NAME("OpenXREmulatedLayerRender"), PassInfo, ERDGPassFlags::Raster,
		[this, RenderParams](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.SetViewport((float)RenderParams.Viewport.Min.X, (float)RenderParams.Viewport.Min.Y, 0.0f, (float)RenderParams.Viewport.Max.X, (float)RenderParams.Viewport.Max.Y, 1.0f);

			if (!EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::BackgroundLayerVisible))
			{
				DrawClearQuad(RHICmdList, FLinearColor::Black);
			}

			FDefaultStereoLayers::StereoLayerRender(RHICmdList, BackgroundCompositedEmulatedLayers, RenderParams);
		});
}

FEmulatedLayersPass* FOpenXRHMD::SetupEmulatedLayersRenderPass(
	FRDGBuilder& GraphBuilder,
	const FSceneView& InView,
	TArray<FDefaultStereoLayers::FStereoLayerToRender>& Layers,
	FRDGTextureRef RenderTarget,
	FDefaultStereoLayers_LayerRenderParams& OutRenderParams)
{
	OutRenderParams = CalculateEmulatedLayerRenderParams(InView);
	FEmulatedLayersPass* Pass = GraphBuilder.AllocParameters<FEmulatedLayersPass>();
	for (const FDefaultStereoLayers::FStereoLayerToRender& Layer : Layers)
	{
		FRDGTextureRef RDGTexture = RegisterExternalTexture(GraphBuilder, Layer.Texture, TEXT("OpenXR_Layer"));
		Pass->LayerTextures.Add(FRDGTextureAccess(RDGTexture, ERHIAccess::SRVGraphics));
	}

	Pass->RenderTargets[0] = FRenderTargetBinding(RenderTarget, ERenderTargetLoadAction::ELoad);
	return Pass;
}

class FEmulatedLayersPass* FOpenXRHMD::SetupEmulatedLayerRenderPass(
	FRDGBuilder& GraphBuilder,
	const FSceneView& InView,
	const FDefaultStereoLayers::FStereoLayerToRender& Layer,
	FRDGTextureRef RenderTarget,
	FRDGTextureRef DepthTarget,
	FDefaultStereoLayers_LayerRenderParams& OutRenderParams)
{
	OutRenderParams = CalculateEmulatedLayerRenderParams(InView);
	FEmulatedLayersPass* Pass = GraphBuilder.AllocParameters<FEmulatedLayersPass>();

	FRDGTextureRef RDGTexture = RegisterExternalTexture(GraphBuilder, Layer.Texture, TEXT("OpenXR_Layer"));
	Pass->LayerTextures.Add(FRDGTextureAccess(RDGTexture, ERHIAccess::SRVGraphics));

	Pass->RenderTargets.DepthStencil = FDepthStencilBinding(
		DepthTarget,
		ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ENoAction,
		FExclusiveDepthStencil::DepthWrite_StencilNop);
	
	Pass->RenderTargets[0] = FRenderTargetBinding(RenderTarget, ERenderTargetLoadAction::ELoad);
	return Pass;
}

FDefaultStereoLayers_LayerRenderParams FOpenXRHMD::CalculateEmulatedLayerRenderParams(const FSceneView& InView)
{
	FViewMatrices ModifiedViewMatrices = InView.ViewMatrices;
	ModifiedViewMatrices.HackRemoveTemporalAAProjectionJitter();
	const FMatrix& ProjectionMatrix = ModifiedViewMatrices.GetViewToClip();
	const FMatrix& ViewProjectionMatrix = ModifiedViewMatrices.GetWorldToClip();

	// Calculate a view matrix that only adjusts for eye position, ignoring head position, orientation and world position.
	FVector EyeShift;
	FQuat EyeOrientation;
	GetRelativeEyePose(IXRTrackingSystem::HMDDeviceId, InView.StereoViewIndex, EyeOrientation, EyeShift);

	FMatrix EyeMatrix = FTranslationMatrix(-EyeShift) * FInverseRotationMatrix(EyeOrientation.Rotator()) * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	FQuat HmdOrientation = FQuat::Identity;
	FVector HmdLocation = FVector::ZeroVector;
	GetCurrentPose(IXRTrackingSystem::HMDDeviceId, HmdOrientation, HmdLocation);

	FMatrix TrackerMatrix = FTranslationMatrix(-HmdLocation) * FInverseRotationMatrix(HmdOrientation.Rotator()) * EyeMatrix;

	FDefaultStereoLayers_LayerRenderParams RenderParams
	{
		InView.UnscaledViewRect, // Viewport
		{
			ViewProjectionMatrix,				// WorldLocked,
			TrackerMatrix * ProjectionMatrix,	// TrackerLocked,
			EyeMatrix * ProjectionMatrix		// FaceLocked
		}
	};
	return RenderParams;
}

void FOpenXRHMD::OnBeginRendering_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& ViewFamily)
{
	ensure(IsInRenderingThread());
	if (!RenderBridge)
	{
		// Frame submission is not necessary in a headless session.
		return;
	}
	
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Module->OnBeginRendering_RenderThread_PreDeviceLocationUpdate(Session, GraphBuilder);
	}
	
	// Snapshot new poses for late update. We either do this here, or queue it from OnBeginRendering_GameThread().
	// If we do it here, it's guaranteed that all platforms will have late update poses available,
	// but planar reflections will be rendered with pre-late update poses, causing them to visually lag behind the rest of the scene.
	if (CVarOpenXRLateUpdateDeviceLocationsAfterReflections.GetValueOnRenderThread())
	{
		UpdateDeviceLocations(false);
	}
	
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Module->OnBeginRendering_RenderThread(Session, GraphBuilder);
	}

	SetupFrameLayers_RenderThread(GraphBuilder);

	const float WorldToMeters = GetWorldToMetersScale();

	if (PipelinedFrameStateRendering.Views.Num() == ViewFamily.Views.Num())
	{
		for (int32 ViewIndex = 0; ViewIndex < ViewFamily.Views.Num(); ViewIndex++)
		{
			if (ViewFamily.Views[ViewIndex]->StereoPass == EStereoscopicPass::eSSP_FULL)
			{
				continue;
			}

			const XrView& View = PipelinedFrameStateRendering.Views[ViewIndex];
			FTransform EyePose = ToFTransform(View.pose, WorldToMeters);

			// Apply the base HMD pose to each eye pose, we will late update this pose for late update in another callback
			FTransform BasePose(ViewFamily.Views[ViewIndex]->BaseHmdOrientation, ViewFamily.Views[ViewIndex]->BaseHmdLocation);
			FTransform BasePoseTransform = EyePose * BasePose;
			BasePoseTransform.NormalizeRotation();

			XrCompositionLayerProjectionView& Projection = PipelinedLayerStateRendering.ProjectionLayers[ViewIndex];
			Projection.pose = ToXrPose(BasePoseTransform, WorldToMeters);
			Projection.fov = View.fov;

			if (EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitEmulatedFaceLockedLayer))
			{
				XrCompositionLayerProjectionView& CompositedProjection = PipelinedLayerStateRendering.EmulatedLayerState.CompositedProjectionLayers[ViewIndex];
				CompositedProjection.pose = ToXrPose(EyePose, WorldToMeters);
				CompositedProjection.fov = View.fov;
			}
		}
	}
	
	if (bHiddenAreaMaskSupported && bNeedReBuildOcclusionMesh)
	{
		BuildOcclusionMeshes();
	}

	// Guard prediction-dependent calls from being invoked (LocateViews, BeginFrame, etc)
	if (bIsRunning && PipelinedFrameStateRendering.bXrFrameStateUpdated)
	{
		// Locate the views we will actually be rendering for.
		// This is required to support late-updating the field-of-view.
        //Note: This LocateViews happens before xrBeginFrame.  Which I don't think is correct.
		LocateViews(PipelinedFrameStateRendering, false);

		SCOPED_NAMED_EVENT(EnqueueFrame, FColor::Red);

		FXRSwapChainPtr ColorSwapchain = PipelinedLayerStateRendering.ColorSwapchain;
		FXRSwapChainPtr DepthSwapchain = PipelinedLayerStateRendering.DepthSwapchain;

		// These swapchains are only present if we are supporting frame synthesis/spacewarp.
		// Always sanity check before using them.
		FXRSwapChainPtr MotionVectorSwapchain = PipelinedLayerStateRendering.MotionVectorSwapchain;
		FXRSwapChainPtr MotionVectorDepthSwapchain = PipelinedLayerStateRendering.MotionVectorDepthSwapchain;

		// This swapchain might not be present depending on the platform support for stereo layers.
		FXRSwapChainPtr EmulationSwapchain = PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain;

		if (bFoveationExtensionSupported && FBFoveationImageGenerator.IsValid())
		{
			FBFoveationImageGenerator->UpdateFoveationImages();
			FBFoveationImageGenerator->SetCurrentFrameSwapchainIndex(ColorSwapchain->GetSwapChainIndex_RHIThread());
		}

		UE_LOGF(LogHMD, VeryVerbose, "%ls WF_%i EnqueueLambda OnBeginRendering_RHIThread", HMDThreadString(), PipelinedFrameStateRendering.WaitCount);
		// For now, leaving swapchain acquisition outside of an RDG pass to make sure the swapchain is acquired early enough.
		GraphBuilder.RHICmdList.EnqueueLambda([this, FrameState = PipelinedFrameStateRendering, ColorSwapchain, DepthSwapchain, MotionVectorSwapchain, MotionVectorDepthSwapchain, EmulationSwapchain](FRHICommandListImmediate& InRHICmdList)
		{
			IRHICommandContext* Context = &InRHICmdList.GetContext();
			RenderBridge->RunOnRHISubmissionThread([this, FrameState = PipelinedFrameStateRendering, ColorSwapchain, DepthSwapchain, MotionVectorSwapchain, MotionVectorDepthSwapchain, EmulationSwapchain, Context]()
			{
				OnBeginRendering_RHIThread(*Context, FrameState, ColorSwapchain, DepthSwapchain, MotionVectorSwapchain, MotionVectorDepthSwapchain, EmulationSwapchain);
			});
		});
	}
}

void FOpenXRHMD::LocateViews(FPipelinedFrameState& PipelineState, bool ResizeViewsArray)
{
	check(PipelineState.bXrFrameStateUpdated);
	FReadScopeLock DeviceLock(DeviceMutex);

	uint32_t ViewCount = 0;
	XrViewLocateInfo ViewInfo;
	ViewInfo.type = XR_TYPE_VIEW_LOCATE_INFO;
	ViewInfo.next = nullptr;
	ViewInfo.viewConfigurationType = SelectedViewConfigurationType;
	ViewInfo.space = DeviceSpaces[HMDDeviceId].Space;
	ViewInfo.displayTime = PipelineState.FrameState.predictedDisplayTime;
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		ViewInfo.next = Module->OnLocateViews(Session, ViewInfo.displayTime, ViewInfo.next);
	}

	XR_ENSURE(xrLocateViews(Session, &ViewInfo, &PipelineState.ViewState, 0, &ViewCount, nullptr));
	if (ResizeViewsArray)
	{
		PipelineState.Views.SetNum(ViewCount, EAllowShrinking::No);
	}
	else
	{
		// PipelineState.Views.Num() can be greater than ViewCount if there is an IOpenXRExtensionPlugin
		// which appends more views with the GetViewLocations callback.
		ensure(PipelineState.Views.Num() >= (int32)ViewCount);
	}
	
	XR_ENSURE(xrLocateViews(Session, &ViewInfo, &PipelineState.ViewState, PipelineState.Views.Num(), &ViewCount, PipelineState.Views.GetData()));
}

void FOpenXRHMD::OnLateUpdateApplied_RenderThread(FRDGBuilder& GraphBuilder, const FTransform& NewRelativeTransform)
{
	FHeadMountedDisplayBase::OnLateUpdateApplied_RenderThread(GraphBuilder, NewRelativeTransform);

	ensure(IsInRenderingThread());

	if (PipelinedFrameStateRendering.Views.Num() == PipelinedLayerStateRendering.ProjectionLayers.Num())
	{
		for (int32 ViewIndex = 0; ViewIndex < PipelinedLayerStateRendering.ProjectionLayers.Num(); ViewIndex++)
		{
			const XrView& View = PipelinedFrameStateRendering.Views[ViewIndex];
			XrCompositionLayerProjectionView& Projection = PipelinedLayerStateRendering.ProjectionLayers[ViewIndex];

			// Apply the new HMD orientation to each eye pose for the final pose
			FTransform EyePose = ToFTransform(View.pose, GetWorldToMetersScale());
			FTransform NewRelativePoseTransform = EyePose * NewRelativeTransform;
			NewRelativePoseTransform.NormalizeRotation();
			Projection.pose = ToXrPose(NewRelativePoseTransform, GetWorldToMetersScale());

			// Update the field-of-view to match the final projection matrix
			Projection.fov = View.fov;

			if (EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitEmulatedFaceLockedLayer))
			{
				XrCompositionLayerProjectionView& CompositedProjection = PipelinedLayerStateRendering.EmulatedLayerState.CompositedProjectionLayers[ViewIndex];
				CompositedProjection.pose = ToXrPose(EyePose, GetWorldToMetersScale());
				CompositedProjection.fov = View.fov;
			}
		}
	}
	
	GraphBuilder.RHICmdList.EnqueueLambda([this, ProjectionLayers = PipelinedLayerStateRendering.ProjectionLayers, CompositedProjectionLayers = PipelinedLayerStateRendering.EmulatedLayerState.CompositedProjectionLayers](FRHICommandListImmediate& InRHICmdList)
	{
		PipelinedLayerStateRHI.ProjectionLayers = ProjectionLayers;
		PipelinedLayerStateRHI.EmulatedLayerState.CompositedProjectionLayers = CompositedProjectionLayers;
	});
}

void FOpenXRHMD::OnBeginRendering_GameThread(FSceneViewFamily& InViewFamily)
{
	// We need to make sure we keep the Wait/Begin/End triplet in sync, so here we signal that we
	// can wait for the next frame in the next tick. Without this signal it's possible that two ticks
	// happen before the next frame is actually rendered.
	bShouldWait = true;

	SetupFrameLayers_GameThread();

    if (bIsReady && bIsRunning)
    {
        for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
        {
            Module->OnBeginRendering_GameThread(Session, InViewFamily, VisibleLayerIds);
        }
    }

	ENQUEUE_RENDER_COMMAND(TransferFrameStateToRenderingThread)(
		[this, GameFrameState = PipelinedFrameStateGame, bBackgroundLayerVisible = IsBackgroundLayerVisible()](FRHICommandListImmediate& RHICmdList) mutable
		{
			UE_CLOGF(PipelinedFrameStateRendering.FrameState.predictedDisplayTime >= GameFrameState.FrameState.predictedDisplayTime,
				LogHMD, VeryVerbose, "Predicted display time went backwards from %lld to %lld", PipelinedFrameStateRendering.FrameState.predictedDisplayTime, GameFrameState.FrameState.predictedDisplayTime);

			UE_LOGF(LogHMD, VeryVerbose, "%ls WF_%i FOpenXRHMD TransferFrameStateToRenderingThread", HMDThreadString(), GameFrameState.WaitCount);
			PipelinedFrameStateRendering = MoveTemp(GameFrameState);

			// Snapshot new poses for late update. We either do this here, or in OnBeginRendering_RenderThread().
			// If we do it here, we'll have the correct late update poses for reflection rendering, but may end up getting the same
			// poses as we had before late update on some platforms because they don't have new poses available yet.
			if (!CVarOpenXRLateUpdateDeviceLocationsAfterReflections.GetValueOnRenderThread())
			{
				UpdateDeviceLocations(false);
			}

			PipelinedLayerStateRendering.LayerStateFlags = EOpenXRLayerStateFlags::None;

			// If we are emulating layers, we still need to submit background layer since we composite into it
			PipelinedLayerStateRendering.LayerStateFlags |= bBackgroundLayerVisible ? EOpenXRLayerStateFlags::BackgroundLayerVisible : EOpenXRLayerStateFlags::None;
			PipelinedLayerStateRendering.LayerStateFlags |= (bBackgroundLayerVisible || IsEmulatingStereoLayers()) ?
				EOpenXRLayerStateFlags::SubmitBackgroundLayer : EOpenXRLayerStateFlags::None;
		});
}

void FOpenXRHMD::OnBeginSimulation_GameThread()
{
	FReadScopeLock Lock(SessionHandleMutex);

	if (!bShouldWait || (!RenderBridge && !bIsTrackingOnlySession))
	{
		return;
	}

	FPipelinedFrameStateAccessorReadWrite LockedPipelineState = GetPipelinedFrameStateForThread();
	FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();
	PipelineState.bXrFrameStateUpdated = false;
	PipelineState.FrameState = { XR_TYPE_FRAME_STATE };

	if (!bIsReady || !bIsRunning)
	{
		return;
	}

	ensure(IsInGameThread());

	SCOPED_NAMED_EVENT(OpenXrWaitFrame, FColor::Red);

	XrFrameWaitInfo WaitInfo;
	WaitInfo.type = XR_TYPE_FRAME_WAIT_INFO;
	WaitInfo.next = nullptr;

	XrFrameState FrameState{XR_TYPE_FRAME_STATE};
	{
		SCOPED_NAMED_EVENT(PluginsOnWaitFrame, FColor::Red);
		for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
		{
			FrameState.next = Module->OnWaitFrame(Session, FrameState.next);
		}
	}
	static int WaitCount = 0;
	++WaitCount;
	UE_LOGF(LogHMD, VeryVerbose, "%ls WF_%i xrWaitFrame Calling...", HMDThreadString(), WaitCount);
	{
		SCOPED_NAMED_EVENT(xrWaitFrame, FColor::Red);
		TRACE_BOOKMARK(TEXT("xrWaitFrame: %d"), WaitCount);
		XR_ENSURE(xrWaitFrame(Session, &WaitInfo, &FrameState));
	}
	UE_LOGF(LogHMD, VeryVerbose, "%ls WF_%i xrWaitFrame Complete", HMDThreadString(), WaitCount);

	// The pipeline state on the game thread can only be safely modified after xrWaitFrame which will be unblocked by
	// the runtime when xrBeginFrame is called. The rendering thread will clone the game pipeline state before calling
	// xrBeginFrame so the game pipeline state can safely be modified after xrWaitFrame returns.

	PipelineState.WaitCount = WaitCount;
	PipelineState.bXrFrameStateUpdated = true;
	PipelineState.FrameState = FrameState;
	PipelineState.WorldToMetersScale = WorldToMetersScale;

	if (bTrackingSpaceInvalid || !ensure(PipelineState.TrackingSpace.IsValid()))
	{
		SCOPED_NAMED_EVENT(CreateTrackingSpace, FColor::Red);		
		// Create the tracking space we'll use until the next recenter.
		FTransform BaseTransform(BaseOrientation, BasePosition);
		PipelineState.TrackingSpace = MakeShared<FTrackingSpace>(TrackingSpaceType, ToXrPose(BaseTransform, WorldToMetersScale));
		PipelineState.TrackingSpace->CreateSpace(Session);
		bTrackingSpaceInvalid = false;
	}

	bShouldWait = false;

	EnumerateViews(PipelineState);
}

bool FOpenXRHMD::ReadNextEvent(XrEventDataBuffer* buffer)
{
	// It is sufficient to clear just the XrEventDataBuffer header to XR_TYPE_EVENT_DATA_BUFFER
	XrEventDataBaseHeader* baseHeader = reinterpret_cast<XrEventDataBaseHeader*>(buffer);
	*baseHeader = { XR_TYPE_EVENT_DATA_BUFFER };
	XrResult Result = XR_ERROR_VALIDATION_FAILURE;
	XR_ENSURE(Result = xrPollEvent(Instance, buffer));
	if (Result == XR_SUCCESS) //-V547  The macro above confuses static analasis into thinking this condition is always false.
	{
		for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
		{
			Module->OnEvent(Session, baseHeader);
		}
		return true;
	}
	return false;
}

bool FOpenXRHMD::OnStartGameFrame(FWorldContext& WorldContext)
{
#if WITH_EDITOR
	// In the editor there can be multiple worlds.  An editor world, pie worlds, other viewport worlds for editor pages.
	// XR hardware can only be running with one of them.
	if (GIsEditor && GEditor && GEditor->GetPIEWorldContext() != nullptr)
	{
		if (!WorldContext.bIsPrimaryPIEInstance && !bIsTrackingOnlySession)
		{
			return false;
		}
	}
#endif // WITH_EDITOR

	if (!System)
	{
		System = IOpenXRHMDModule::Get().GetSystemId();
		if (System)
		{
			FCoreDelegates::VRHeadsetReconnected.Broadcast();
		}
		else if (Session == XR_NULL_HANDLE)
		{
			// Having a session but no system does not make much sense, but we will continue to process XrEvents just in case.
			return false;
		}
	}

	const AWorldSettings* const WorldSettings = WorldContext.World() ? WorldContext.World()->GetWorldSettings() : nullptr;
	if (WorldSettings)
	{
		WorldToMetersScale = WorldSettings->WorldToMeters;
	}

	RefreshTrackingToWorldTransform(WorldContext);

	if (bIsTrackingOnlySession)
	{
		if (OnStereoStartup())
		{
			StartSession();
		}
	}

	// Process all pending messages.
	XrEventDataBuffer event;
	while (ReadNextEvent(&event))
	{
		switch (event.type)
		{
		case XR_TYPE_EVENT_DATA_SESSION_STATE_CHANGED:
		{
			const XrEventDataSessionStateChanged& SessionState =
				reinterpret_cast<XrEventDataSessionStateChanged&>(event);

			CurrentSessionState = SessionState.state;

			UE_LOGF(LogHMD, Verbose, "Session state switching to %ls", OpenXRSessionStateToString(CurrentSessionState));

			if (SessionState.state == XR_SESSION_STATE_READY)
			{
				if (!GIsEditor)
				{
					GEngine->SetMaxFPS(0);
				}
				bIsReady = true;
				StartSession();
			}
			else if (SessionState.state == XR_SESSION_STATE_SYNCHRONIZED)
			{
				bIsSynchronized = true;
			}
			else if (SessionState.state == XR_SESSION_STATE_IDLE)
			{
				bIsSynchronized = false;
			}
			else if (SessionState.state == XR_SESSION_STATE_STOPPING)
			{
				if (!GIsEditor)
				{
					const int32 PausedIdleFPS = CVarOpenXRPausedIdleFPS.GetValueOnAnyThread();
					GEngine->SetMaxFPS(PausedIdleFPS);
				}
				bIsReady = false;
				StopSession();
			}
			else if (SessionState.state == XR_SESSION_STATE_EXITING || SessionState.state == XR_SESSION_STATE_LOSS_PENDING)
			{
				// We need to make sure we unlock the frame rate again when exiting stereo while idle
				if (!GIsEditor)
				{
					GEngine->SetMaxFPS(0);
				}

				if (SessionState.state == XR_SESSION_STATE_LOSS_PENDING)
				{
					FCoreDelegates::VRHeadsetLost.Broadcast();
					System = XR_NULL_SYSTEM_ID;
				}
				
				FApp::SetHasVRFocus(false);

				DestroySession();

				// Do we want to RequestExitApp the app after destoying the session?
				// Yes if the app (ie ue4) did NOT requested the exit.
				bool bExitApp = !bIsExitingSessionByxrRequestExitSession;
				bIsExitingSessionByxrRequestExitSession = false;

				// But only if this CVar is set to true.
				bExitApp = bExitApp && (CVarOpenXRExitAppOnRuntimeDrivenSessionExit.GetValueOnAnyThread() != 0);
	
				if (bExitApp)
				{
					RequestExitApp();
				}
				break;
			}

			FApp::SetHasVRFocus(SessionState.state == XR_SESSION_STATE_FOCUSED);
			
			break;
		}
		case XR_TYPE_EVENT_DATA_INSTANCE_LOSS_PENDING:
		{
			DestroySession();
			
			// Instance loss is intended to support things like updating the active openxr runtime.  Currently we just require an app restart.
			RequestExitApp();

			break;
		}
		case XR_TYPE_EVENT_DATA_REFERENCE_SPACE_CHANGE_PENDING:
		{
			const XrEventDataReferenceSpaceChangePending& SpaceChange =
				reinterpret_cast<XrEventDataReferenceSpaceChangePending&>(event);
			check(SpaceChange.type == XR_TYPE_EVENT_DATA_REFERENCE_SPACE_CHANGE_PENDING);

			if (SpaceChange.referenceSpaceType == XR_REFERENCE_SPACE_TYPE_STAGE)
			{
				OnPlayAreaChanged();
			}

			FCoreDelegates::VRHeadsetRecenter.Broadcast();
			break;
		}
		case XR_TYPE_EVENT_DATA_INTERACTION_PROFILE_CHANGED:
		{
			OnInteractionProfileChanged();
			break;
		}
		case XR_TYPE_EVENT_DATA_VISIBILITY_MASK_CHANGED_KHR:
		{
			bHiddenAreaMaskSupported = ensure(IsExtensionEnabled(XR_KHR_VISIBILITY_MASK_EXTENSION_NAME));  // Ensure fail indicates a non-conformant openxr implementation.
			bNeedReBuildOcclusionMesh = true;
			break;
		}
		case XR_TYPE_EVENT_DATA_USER_PRESENCE_CHANGED_EXT:
		{
			const XrEventDataUserPresenceChangedEXT& PresenceChanged = 
				reinterpret_cast<XrEventDataUserPresenceChangedEXT&>(event);
			check(PresenceChanged.type == XR_TYPE_EVENT_DATA_USER_PRESENCE_CHANGED_EXT);

			if (PresenceChanged.isUserPresent)
			{
				FCoreDelegates::VRHeadsetPutOnHead.Broadcast();
			}
			else
			{
				FCoreDelegates::VRHeadsetRemovedFromHead.Broadcast();
			}
			break;
		}
		}
	}

	GetARCompositionComponent()->StartARGameFrame(WorldContext);

	// TODO: We could do this earlier in the pipeline and allow simulation to run one frame ahead of the render thread.
	// That would allow us to take more advantage of Late Update and give projects more headroom for simulation.
	// However currently blocking in earlier callbacks can result in a pipeline stall, so we do it here instead.
	OnBeginSimulation_GameThread();

	// Snapshot new poses for game simulation.
	UpdateDeviceLocations(true);

	return true;
}

bool FOpenXRHMD::SetColorScaleAndBias(FLinearColor ColorScale, FLinearColor ColorBias)
{
	if (!bCompositionLayerColorScaleBiasSupported)
	{
		return false;
	}

	LayerColorScale = XrColor4f{ ColorScale.R, ColorScale.G, ColorScale.B, ColorScale.A };
	LayerColorBias = XrColor4f{ ColorBias.R, ColorBias.G, ColorBias.B, ColorBias.A };
	return true;
}

TArray<FTextureRHIRef, TInlineAllocator<2>> FOpenXRHMD::GetDebugLayerTextures_RenderThread()
{
	TArray<FTextureRHIRef, TInlineAllocator<2>> DebugLayers;
	for (const FOpenXRLayer& Layer : NativeLayers)
	{
		if (Layer.Desc.Flags & LAYER_FLAG_DEBUG &&
			Layer.RightEye.Texture.IsValid() &&
			Layer.RightEye.Texture->GetTexture2D() != nullptr)
		{
			DebugLayers.Add(Layer.RightEye.Texture);
		}
	}
	return DebugLayers;
}

void FOpenXRHMD::RequestExitApp()
{
	UE_LOGF(LogHMD, Log, "FOpenXRHMD is requesting app exit.  CurrentSessionState: %ls", OpenXRSessionStateToString(CurrentSessionState));

#if WITH_EDITOR
	if (GIsEditor)
	{
		FSceneViewport* SceneVP = FindSceneViewport();
		if (SceneVP && SceneVP->IsStereoRenderingAllowed())
		{
			TSharedPtr<SWindow> Window = SceneVP->FindWindow();
			Window->RequestDestroyWindow();
		}
	}
	else
#endif//WITH_EDITOR
	{
		// ApplicationWillTerminateDelegate will fire from inside of the RequestExit
		FPlatformMisc::RequestExit(false);
	}
}

void FOpenXRHMD::OnBeginRendering_RHIThread(IRHICommandContext& RHICmdContext, const FPipelinedFrameState& InFrameState, FXRSwapChainPtr ColorSwapchain, FXRSwapChainPtr DepthSwapchain, FXRSwapChainPtr MotionVectorSwapchain, FXRSwapChainPtr MotionVectorDepthSwapchain, FXRSwapChainPtr EmulationSwapchain)
{
	// TODO: Add a hook to resolve discarded frames before we start a new frame.
	UE_CLOGF(bIsRendering, LogHMD, Verbose, "Discarded previous frame and started rendering a new frame.");

	SCOPED_NAMED_EVENT(BeginFrame, FColor::Red);

	FReadScopeLock Lock(SessionHandleMutex);
	if (!bIsRunning || (!RenderBridge && !bIsTrackingOnlySession))
	{
		return;
	}

	// We do not want xrBeginFrame to run twice based on a single xrWaitFrame.
	// During LoadMap RedrawViewports(false) is called twice to pump the render thread without a new game thread pump.  This results in this function being
	// called two additional times without corresponding xrWaitFrame calls from the game thread and therefore two extra xrBeginFrame calls.  On SteamVR, at least,
	// this then leaves us in a situation where our xrWaitFrame immediately returns forever.
	// To avoid this we ensure that each xrWaitFrame is consumed by xrBeginFrame only once.  We use the count of xrWaitFrame calls as an identifier.  Before 
	// xrBeginFrame if the PipelinedFrameStateRHI wait count equals the incoming pipelined xrWaitFrame count then that xrWaitFrame has already been consumed,
	// so we early out.  Once a new game frame happens and a new xrWaitFrame the early out will fail and xrBeginFrame will happen.
	if ((PipelinedFrameStateRHI.WaitCount == InFrameState.WaitCount) && bUseWaitCountToAvoidExtraXrBeginFrameCalls)
	{
		UE_LOGF(LogHMD, Verbose, "FOpenXRHMD::OnBeginRendering_RHIThread returning before xrBeginFrame because xrWaitFrame %i is already consumed.  This is expected twice during LoadMap and may also happen during other 'extra' render pumps.", InFrameState.WaitCount);
		return;
	}

	// The layer state will be copied after SetFinalViewRect
	PipelinedFrameStateRHI = InFrameState;

	void* Next = nullptr;
	XrRHIContextEPIC RHIContextEPIC = { (XrStructureType)XR_TYPE_RHI_CONTEXT_EPIC };
	if (RuntimeRequiresRHIContext())
	{
		RHIContextEPIC.RHIContext = &RHICmdContext;
		RHIContextEPIC.next = Next;
		Next = &RHIContextEPIC;
	}
	XrFrameBeginInfo BeginInfo;
	BeginInfo.type = XR_TYPE_FRAME_BEGIN_INFO;
	BeginInfo.next = Next;
	XrTime DisplayTime = PipelinedFrameStateRHI.FrameState.predictedDisplayTime;
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		BeginInfo.next = Module->OnBeginFrame_RHIThread(Session, DisplayTime, BeginInfo.next);
	}
	static int BeginCount = 0;
	PipelinedFrameStateRHI.BeginCount = ++BeginCount;

	TRACE_BOOKMARK(TEXT("xrBeginFrame: %d"), BeginCount);
	UE_LOGF(LogHMD, VeryVerbose, "%ls WF_%i xrBeginFrame BeginCount: %i", HMDThreadString(), PipelinedFrameStateRHI.WaitCount, PipelinedFrameStateRHI.BeginCount);
	XrResult Result = xrBeginFrame(Session, &BeginInfo);
	if (XR_SUCCEEDED(Result))
	{
		// Only the swapchains are valid to pull out of PipelinedLayerStateRendering
		// Full population is deferred until SetFinalViewRect.
		// TODO Possibly move these Waits to SetFinalViewRect??
		PipelinedLayerStateRHI.ColorSwapchain = ColorSwapchain;
		PipelinedLayerStateRHI.DepthSwapchain = DepthSwapchain;
		PipelinedLayerStateRHI.MotionVectorSwapchain = MotionVectorSwapchain;
		PipelinedLayerStateRHI.MotionVectorDepthSwapchain = MotionVectorDepthSwapchain;
		PipelinedLayerStateRHI.EmulatedLayerState.EmulationSwapchain = EmulationSwapchain;

		// We need a new swapchain image unless we've already acquired one for rendering
		if (!bIsRendering && ColorSwapchain)
		{
			ColorSwapchain->WaitCurrentImage_RHIThread(OPENXR_SWAPCHAIN_WAIT_TIMEOUT);
			if (!bIsAcquireOnAnyThreadSupported)
			{
				ColorSwapchain->IncrementSwapChainIndex_RHIThread();
			}
			if (DepthSwapchain)
			{
				DepthSwapchain->WaitCurrentImage_RHIThread(OPENXR_SWAPCHAIN_WAIT_TIMEOUT);
				if (!bIsAcquireOnAnyThreadSupported)
				{
					DepthSwapchain->IncrementSwapChainIndex_RHIThread();
				}
			}
			if (EmulationSwapchain)
			{
				EmulationSwapchain->WaitCurrentImage_RHIThread(OPENXR_SWAPCHAIN_WAIT_TIMEOUT);
				if (!bIsAcquireOnAnyThreadSupported)
				{
					EmulationSwapchain->IncrementSwapChainIndex_RHIThread();
				}
			}
			if (MotionVectorSwapchain)
			{
				MotionVectorSwapchain->WaitCurrentImage_RHIThread(OPENXR_SWAPCHAIN_WAIT_TIMEOUT);
				if (!bIsAcquireOnAnyThreadSupported)
				{
					MotionVectorSwapchain->IncrementSwapChainIndex_RHIThread();
				}
			}
			if (MotionVectorDepthSwapchain)
			{
				checkf(MotionVectorSwapchain, TEXT("Motion vector depth swapchain should not exist without a motion vector swapchain."));

				MotionVectorDepthSwapchain->WaitCurrentImage_RHIThread(OPENXR_SWAPCHAIN_WAIT_TIMEOUT);
				if (!bIsAcquireOnAnyThreadSupported)
				{
					MotionVectorDepthSwapchain->IncrementSwapChainIndex_RHIThread();
				}
			}
		}

		bIsRendering = true;

		UE_LOGF(LogHMD, VeryVerbose, "%ls WF_%i Rendering frame predicted to be displayed at %lld", 
			   HMDThreadString(), PipelinedFrameStateRHI.WaitCount,
			   PipelinedFrameStateRHI.FrameState.predictedDisplayTime);

		for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
		{
			Module->PostBeginFrame_RHIThread(PipelinedFrameStateRHI.FrameState.predictedDisplayTime);
		}
	}
	else
	{
		static bool bLoggedBeginFrameFailure = false;
		if (!bLoggedBeginFrameFailure)
		{
			UE_LOGF(LogHMD, Error, "Unexpected error on xrBeginFrame. Error code was %ls.", OpenXRResultToString(Result));
			bLoggedBeginFrameFailure = true;
		}
	}
}

void FOpenXRHMD::OnBackBufferReadyToPresent(SWindow& Window, ISlateViewportProvider& ViewportProvider)
{
	// Hook the present for this window
	if (Window.GetViewport() && Window.GetViewport()->IsStereoscopic3D())
	{
		ViewportProvider.SetCustomPresent(RenderBridge);
	}
}

void FOpenXRHMD::OnFinishRendering(IRHICommandContext* RHICmdContext)
{
	SCOPED_NAMED_EVENT(EndFrame, FColor::Red);

	if (!bIsRendering || !RenderBridge)
	{
		return;
	}
	
	UE_LOGF(LogHMD, VeryVerbose, "%ls WF_%i FOpenXRHMD::OnFinishRendering releasing swapchain images now.", HMDThreadString(), PipelinedFrameStateRHI.WaitCount);

	// We need to ensure we release the swap chain images even if the session is not running.
	if (PipelinedLayerStateRHI.ColorSwapchain)
	{
		IRHICommandContext* const RHICommandContextIfRequired = RuntimeRequiresRHIContext() ? RHICmdContext : nullptr;
		PipelinedLayerStateRHI.ColorSwapchain->ReleaseCurrentImage_RHIThread(RHICommandContextIfRequired);

		if (PipelinedLayerStateRHI.DepthSwapchain)
		{
			PipelinedLayerStateRHI.DepthSwapchain->ReleaseCurrentImage_RHIThread(RHICommandContextIfRequired);
		}
		if (PipelinedLayerStateRHI.MotionVectorSwapchain)
		{
			PipelinedLayerStateRHI.MotionVectorSwapchain->ReleaseCurrentImage_RHIThread(RHICommandContextIfRequired);
		}
		if (PipelinedLayerStateRHI.MotionVectorDepthSwapchain)
		{
			PipelinedLayerStateRHI.MotionVectorDepthSwapchain->ReleaseCurrentImage_RHIThread(RHICommandContextIfRequired);
		}
		if (PipelinedLayerStateRHI.EmulatedLayerState.EmulationSwapchain)
		{
			PipelinedLayerStateRHI.EmulatedLayerState.EmulationSwapchain->ReleaseCurrentImage_RHIThread(RHICommandContextIfRequired);
		}
	}

	FReadScopeLock Lock(SessionHandleMutex);
	if (bIsRunning)
	{
		TArray<XrCompositionLayerBaseHeader*> Headers;

		// Layers are rendered in the order they're submitted to the compositor, so earlier layers in the Headers list will appear below later layers.
		// So we add underlays, then projection layers, then overlays. Underlays will appear below the projection layers, and overlays will appear above.
		for (FXrCompositionLayerUnion& NativeUnderlay : PipelinedLayerStateRHI.NativeUnderlays)
		{
			Headers.Add(reinterpret_cast<XrCompositionLayerBaseHeader*>(&NativeUnderlay.Header));
		}

		XrCompositionLayerProjection BackgroundLayer = {};
		XrCompositionLayerAlphaBlendFB LayerAlphaBlend = { XR_TYPE_COMPOSITION_LAYER_ALPHA_BLEND_FB };
		XrCompositionLayerColorScaleBiasKHR ColorScaleBias = { XR_TYPE_COMPOSITION_LAYER_COLOR_SCALE_BIAS_KHR };
		XrCompositionLayerDepthTestFB LayerDepthTest = { XR_TYPE_COMPOSITION_LAYER_DEPTH_TEST_FB };
		if (EnumHasAnyFlags(PipelinedLayerStateRHI.LayerStateFlags, EOpenXRLayerStateFlags::SubmitBackgroundLayer))
		{
			BackgroundLayer.type = XR_TYPE_COMPOSITION_LAYER_PROJECTION;
			BackgroundLayer.next = nullptr;
			BackgroundLayer.layerFlags = bProjectionLayerAlphaEnabled ? XR_COMPOSITION_LAYER_BLEND_TEXTURE_SOURCE_ALPHA_BIT : 0;
			BackgroundLayer.space = PipelinedFrameStateRHI.TrackingSpace->Handle;
			BackgroundLayer.viewCount = PipelinedLayerStateRHI.ProjectionLayers.Num();
			BackgroundLayer.views = PipelinedLayerStateRHI.ProjectionLayers.GetData();

			Headers.Add(reinterpret_cast<XrCompositionLayerBaseHeader*>(&BackgroundLayer));

			if (CVarOpenXRInvertAlpha.GetValueOnRenderThread())
			{
				// These two extensions do the same thing, the first one is more modern.  The second is to keep older runtimes functioning.
				if(IsExtensionEnabled(XR_EXT_COMPOSITION_LAYER_INVERTED_ALPHA_EXTENSION_NAME))
				{
					BackgroundLayer.layerFlags |= XR_COMPOSITION_LAYER_INVERTED_ALPHA_BIT_EXT;
				}
				else if (IsExtensionEnabled(XR_FB_COMPOSITION_LAYER_ALPHA_BLEND_EXTENSION_NAME))
				{
					LayerAlphaBlend.next = const_cast<void*>(BackgroundLayer.next);
					LayerAlphaBlend.srcFactorColor = PipelinedLayerStateRHI.BasePassLayerBlendParams.srcFactorColor;
					LayerAlphaBlend.dstFactorColor = PipelinedLayerStateRHI.BasePassLayerBlendParams.dstFactorColor;
					LayerAlphaBlend.srcFactorAlpha = PipelinedLayerStateRHI.BasePassLayerBlendParams.srcFactorAlpha;
					LayerAlphaBlend.dstFactorAlpha = PipelinedLayerStateRHI.BasePassLayerBlendParams.dstFactorAlpha;

					BackgroundLayer.next = &LayerAlphaBlend;
				}
			}

			if (bCompositionLayerColorScaleBiasSupported)
			{
				ColorScaleBias.next = const_cast<void*>(BackgroundLayer.next);
				ColorScaleBias.colorScale = PipelinedLayerStateRHI.LayerColorScaleAndBias.ColorScale;
				ColorScaleBias.colorBias = PipelinedLayerStateRHI.LayerColorScaleAndBias.ColorBias;

				BackgroundLayer.next = &ColorScaleBias;
			}

			if (IsExtensionEnabled(XR_FB_COMPOSITION_LAYER_DEPTH_TEST_EXTENSION_NAME) && 
				EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitDepthLayer))
			{
				LayerDepthTest.next = const_cast<void*>(BackgroundLayer.next);
				LayerDepthTest.depthMask = true;
				LayerDepthTest.compareOp = XR_COMPARE_OP_LESS_FB;
				BackgroundLayer.next = &LayerDepthTest;
			}

			for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
			{
				BackgroundLayer.next = Module->OnEndProjectionLayer_RHIThread(Session, 0, BackgroundLayer.next, BackgroundLayer.layerFlags);
			}
		}
		
		XrCompositionLayerProjection CompositedLayer = {};
		if (EnumHasAnyFlags(PipelinedLayerStateRHI.LayerStateFlags, EOpenXRLayerStateFlags::SubmitEmulatedFaceLockedLayer))
		{
			CompositedLayer.type = XR_TYPE_COMPOSITION_LAYER_PROJECTION;
			CompositedLayer.next = nullptr;
			// Alpha always enabled to allow for transparency between the composited layers.
			CompositedLayer.layerFlags = XR_COMPOSITION_LAYER_BLEND_TEXTURE_SOURCE_ALPHA_BIT;
			{
				FReadScopeLock DeviceLock(DeviceMutex);
				CompositedLayer.space = DeviceSpaces[HMDDeviceId].Space;
			}
			CompositedLayer.viewCount = PipelinedLayerStateRHI.EmulatedLayerState.CompositedProjectionLayers.Num();
			CompositedLayer.views = PipelinedLayerStateRHI.EmulatedLayerState.CompositedProjectionLayers.GetData();
			Headers.Add(reinterpret_cast<XrCompositionLayerBaseHeader*>(&CompositedLayer));
		}

		for (FXrCompositionLayerUnion& NativeOverlay : PipelinedLayerStateRHI.NativeOverlays)
		{
			Headers.Add(reinterpret_cast<XrCompositionLayerBaseHeader*>(&NativeOverlay.Header));
		}

		for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
		{
			Module->UpdateCompositionLayers_RHIThread(Session, Headers);
		}

		void* Next = nullptr;
		XrRHIContextEPIC RHIContextEPIC = { (XrStructureType)XR_TYPE_RHI_CONTEXT_EPIC };
		if (RuntimeRequiresRHIContext())
		{
			RHIContextEPIC.RHIContext = RHICmdContext;
			RHIContextEPIC.next = Next;
			Next = &RHIContextEPIC;
		}

		int32 BlendModeOverride = CVarOpenXREnvironmentBlendMode.GetValueOnRenderThread();

		XrFrameEndInfo EndInfo;
		EndInfo.type = XR_TYPE_FRAME_END_INFO;
		EndInfo.next = Next;
		EndInfo.displayTime = PipelinedFrameStateRHI.FrameState.predictedDisplayTime;
		EndInfo.environmentBlendMode = BlendModeOverride ? (XrEnvironmentBlendMode)BlendModeOverride : SelectedEnvironmentBlendMode;

		EndInfo.layerCount = PipelinedFrameStateRHI.FrameState.shouldRender ? Headers.Num() : 0;
		EndInfo.layers = PipelinedFrameStateRHI.FrameState.shouldRender ? Headers.GetData() : nullptr;

		for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
		{
			EndInfo.next = Module->OnEndFrame(Session, EndInfo.displayTime, EndInfo.next);
		}

		UE_LOGF(LogHMD, VeryVerbose, "Presenting frame predicted to be displayed at %lld", PipelinedFrameStateRHI.FrameState.predictedDisplayTime);

#if PLATFORM_ANDROID
		// Android OpenXR runtimes frequently need access to the JNIEnv, so we need to attach the submitting
		// thread. We have to do this per-frame because we can detach if app loses focus.
		FAndroidApplication::GetJavaEnv();
#endif
		static int EndCount = 0;
		PipelinedFrameStateRHI.EndCount = ++EndCount;

		TRACE_BOOKMARK(TEXT("xrEndFrame: %d"), EndCount);
		UE_LOGF(LogHMD, VeryVerbose, "%ls WF_%i xrEndFrame WaitCount: %i BeginCount: %i EndCount: %i", HMDThreadString(), PipelinedFrameStateRHI.WaitCount, PipelinedFrameStateRHI.WaitCount, PipelinedFrameStateRHI.BeginCount, PipelinedFrameStateRHI.EndCount);
		XR_ENSURE(xrEndFrame(Session, &EndInfo));
	}

	bIsRendering = false;
}

bool FOpenXRHMD::HDRGetMetaDataForStereo(EDisplayOutputFormat& OutDisplayOutputFormat, EDisplayColorGamut& OutDisplayColorGamut, bool& OutbHDRSupported)
{
	if (RenderBridge == nullptr)
	{
		return false;
	}

	return RenderBridge->HDRGetMetaDataForStereo(OutDisplayOutputFormat, OutDisplayColorGamut, OutbHDRSupported);
}

float FOpenXRHMD::GetPixelDenity() const
{
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();
	return PipelineState.PixelDensity;
}

void FOpenXRHMD::SetPixelDensity(const float NewDensity)
{
	check(IsInGameThread());
	PipelinedFrameStateGame.PixelDensity = FMath::Min(NewDensity, RuntimePixelDensityMax);

	// We have to update the RT state because the new swapchain will be allocated (FSceneViewport::InitRHI + AllocateRenderTargetTexture)
	// before we call OnBeginRendering_GameThread.
	ENQUEUE_RENDER_COMMAND(UpdatePixelDensity)(
		[this, PixelDensity = PipelinedFrameStateGame.PixelDensity](FRHICommandListImmediate&)
		{
			PipelinedFrameStateRendering.PixelDensity = PixelDensity;
		});
}

FIntPoint FOpenXRHMD::GetIdealRenderTargetSize() const
{
	FPipelinedFrameStateAccessorReadOnly LockedPipelineState = GetPipelinedFrameStateForThread();
	const FPipelinedFrameState& PipelineState = LockedPipelineState.GetFrameState();

	FIntPoint Size(EForceInit::ForceInitToZero);
	for (int32 ViewIndex = 0; ViewIndex < PipelineState.ViewConfigs.Num(); ViewIndex++)
	{
		const XrViewConfigurationView& Config = PipelineState.ViewConfigs[ViewIndex];

		// If Mobile Multi-View is active the first two views will share the same position
		Size.X = bIsMobileMultiViewEnabled && ViewIndex < 2 ? FMath::Max(Size.X, (int)Config.recommendedImageRectWidth)
			: Size.X + (int)Config.recommendedImageRectWidth;
		Size.Y = FMath::Max(Size.Y, (int)Config.recommendedImageRectHeight);

		// Make sure we quantize in order to be consistent with the rest of the engine in creating our buffers.
		QuantizeSceneBufferSize(Size, Size);
	}

	return Size;
}

FIntRect FOpenXRHMD::GetFullFlatEyeRect_RenderThread(const FRHITextureDesc& EyeTexture) const
{
	FVector2D SrcNormRectMin(0.05f, 0.2f);
	// with MMV, each eye occupies the whole RT layer, so we don't need to limit the source rect to the left half of the RT.
	FVector2D SrcNormRectMax(bIsMobileMultiViewEnabled ? 0.95f : 0.45f, 0.8f);
	if (!bIsMobileMultiViewEnabled && GetDesiredNumberOfViews(bStereoEnabled) > 2)
	{
		SrcNormRectMin.X /= 2;
		SrcNormRectMax.X /= 2;
	}

	return FIntRect(EyeTexture.GetSize().X * SrcNormRectMin.X, EyeTexture.GetSize().Y * SrcNormRectMin.Y, EyeTexture.GetSize().X * SrcNormRectMax.X, EyeTexture.GetSize().Y * SrcNormRectMax.Y);
}

void FOpenXRHMD::CopySwapchainTexture_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef SrcTexture, FIntRect SrcRect, const FXRSwapChainPtr& DstSwapChain,
	FIntRect DstRect, bool bClearBlack, EXRCopyTextureBlendModifier SrcTextureCopyModifier, FStaticFeatureLevel FeatureLevel, FStaticShaderPlatform ShaderPlatform) const
{
	const bool bAcquireOnRender = bIsAcquireOnAnyThreadSupported;
	if (bAcquireOnRender)
	{
		DstSwapChain->IncrementSwapChainIndex_RHIThread();
	}

	AddPass(GraphBuilder, RDG_EVENT_NAME("OpenXRHMD_AcquireLayerSwapchain"), [this, bAcquireOnRender, DstSwapChain](FRHICommandListImmediate& RHICmdList)
		{
		RHICmdList.EnqueueLambda([this, bAcquireOnRender, DstSwapChain](FRHICommandListImmediate& InRHICmdList)
		{
			RenderBridge->RunOnRHISubmissionThread([bAcquireOnRender, DstSwapChain]()
			{
				if (!bAcquireOnRender)
				{
					DstSwapChain->IncrementSwapChainIndex_RHIThread();
				}
				DstSwapChain->WaitCurrentImage_RHIThread(OPENXR_SWAPCHAIN_WAIT_TIMEOUT);
			});
		});
	});

	// Now that we've enqueued the swapchain wait we can add the commands to do the actual texture copy
	FRHITexture* const DstTexture = DstSwapChain->GetTexture();
	FRDGTextureRef DstTextureRDG = RegisterExternalTexture(GraphBuilder, DstTexture, TEXT("OpenXR_Layer_Swapchain"));

	FXRCopyTextureOptions Options(FeatureLevel, ShaderPlatform);
	Options.bClearBlack = bClearBlack;
	Options.BlendMod = SrcTextureCopyModifier;
	Options.LoadAction = ERenderTargetLoadAction::EClear;
	Options.bOutputMipChain = true;
	// N.B. Don't configure display mapping here, we don't want it.
	AddXRCopyTexturePass(GraphBuilder, RDG_EVENT_NAME("OpenXRHMD_UpdateLayerSwapchain"), SrcTexture, SrcRect, DstTextureRDG, DstRect, Options);

	// Enqueue a command to release the image after the copy is done
	bool bCapturableRequiresRHIContext = RuntimeRequiresRHIContext();
	AddPass(GraphBuilder, RDG_EVENT_NAME("OpenXRHMD_ReleaseLayerSwapchain"), [this, DstSwapChain, bCapturableRequiresRHIContext](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.EnqueueLambda([this, DstSwapChain, bCapturableRequiresRHIContext](FRHICommandListImmediate& InRHICmdList)
		{
			RenderBridge->RunOnRHISubmissionThread([DstSwapChain, bCapturableRequiresRHIContext, &InRHICmdList]()
			{
				DstSwapChain->ReleaseCurrentImage_RHIThread(bCapturableRequiresRHIContext ? &InRHICmdList.GetContext() : nullptr);
			});
		});
	});
}

void FOpenXRHMD::RenderTexture_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef BackBuffer, FRDGTextureRef SrcTexture, FVector2f WindowSize) const
{
	if (SpectatorScreenController)
	{
		const bool bShouldPassLayersTexture =
			EnumHasAnyFlags(PipelinedLayerStateRendering.LayerStateFlags, EOpenXRLayerStateFlags::SubmitEmulatedFaceLockedLayer) &&
			!CVarOpenXRDoNotCopyEmulatedLayersToSpectatorScreen.GetValueOnRenderThread();
		
		const FRDGTextureRef LayersTexture =
			bShouldPassLayersTexture ?
			RegisterExternalTexture(GraphBuilder, PipelinedLayerStateRendering.EmulatedLayerState.EmulationSwapchain->GetTextureRef(), TEXT("OpenXRLayersTexture")) :
			nullptr;
		
		SpectatorScreenController->RenderSpectatorScreen_RenderThread(GraphBuilder, BackBuffer, SrcTexture, LayersTexture, WindowSize);
	}
}

bool FOpenXRHMD::HasHiddenAreaMesh() const
{
	return HiddenAreaMeshes.Num() > 0;
}

bool FOpenXRHMD::HasVisibleAreaMesh() const
{
	return VisibleAreaMeshes.Num() > 0;
}

void FOpenXRHMD::DrawHiddenAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex, int32 InstanceCount) const
{
	check(ViewIndex != INDEX_NONE);

	if (ViewIndex < HiddenAreaMeshes.Num())
	{
		const FHMDViewMesh& Mesh = HiddenAreaMeshes[ViewIndex];

		if (Mesh.IsValid())
		{
			RHICmdList.SetStreamSource(0, Mesh.VertexBufferRHI, 0);
			RHICmdList.DrawIndexedPrimitive(Mesh.IndexBufferRHI, 0, 0, Mesh.NumVertices, 0, Mesh.NumTriangles, InstanceCount);
		}
	}
}

void FOpenXRHMD::DrawVisibleAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex, int32 InstanceCount) const
{
	check(ViewIndex != INDEX_NONE);
	check(ViewIndex < VisibleAreaMeshes.Num());

	if (ViewIndex < VisibleAreaMeshes.Num() && VisibleAreaMeshes[ViewIndex].IsValid())
	{
		const FHMDViewMesh& Mesh = VisibleAreaMeshes[ViewIndex];

		RHICmdList.SetStreamSource(0, Mesh.VertexBufferRHI, 0);
		RHICmdList.DrawIndexedPrimitive(Mesh.IndexBufferRHI, 0, 0, Mesh.NumVertices, 0, Mesh.NumTriangles, InstanceCount);
	}
	else
	{
		// Invalid mesh means that entire area is visible, draw a fullscreen quad to simulate
		FPixelShaderUtils::DrawFullscreenQuad(RHICmdList, 1);
	}
}

void FOpenXRHMD::DrawHiddenAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex) const
{
	DrawHiddenAreaMesh(RHICmdList, ViewIndex, 1);
}

void FOpenXRHMD::DrawVisibleAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex) const
{
	DrawVisibleAreaMesh(RHICmdList, ViewIndex, 1);
}

uint32 FOpenXRHMD::CreateLayer(const FLayerDesc& InLayerDesc)
{
	const uint32_t LayerId = FSimpleLayerManager::CreateLayer(InLayerDesc);

	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Module->OnCreateLayer(LayerId);
	}

	return LayerId;
}

void FOpenXRHMD::DestroyLayer(uint32 LayerId)
{
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Module->OnDestroyLayer(LayerId);
	}

	FSimpleLayerManager::DestroyLayer(LayerId);
}

void FOpenXRHMD::SetLayerDesc(uint32 LayerId, const FLayerDesc& InLayerDesc)
{
	FSimpleLayerManager::SetLayerDesc(LayerId, InLayerDesc);

	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Module->OnSetLayerDesc(LayerId);
	}
}

FOpenXRSwapchain* FOpenXRHMD::GetColorSwapchain_RenderThread()
{
	if (PipelinedLayerStateRendering.ColorSwapchain != nullptr)
	{
		return static_cast<FOpenXRSwapchain*>(PipelinedLayerStateRendering.ColorSwapchain.Get());
	}

	return nullptr;
}

//---------------------------------------------------
// OpenXR Action Space Implementation
//---------------------------------------------------

FOpenXRHMD::FDeviceSpace::FDeviceSpace(XrAction InAction, XrPath InPath)
	: Action(InAction)
	, Space(XR_NULL_HANDLE)
	, Path(InPath)
	, SubactionPath(XR_NULL_PATH)
{
}

FOpenXRHMD::FDeviceSpace::FDeviceSpace(XrAction InAction, XrPath InPath, XrPath InSubactionPath)
	: Action(InAction)
	, Space(XR_NULL_HANDLE)
	, Path(InPath)
	, SubactionPath(InSubactionPath)
{
}

FOpenXRHMD::FDeviceSpace::~FDeviceSpace()
{
	DestroySpace();
}

bool FOpenXRHMD::FDeviceSpace::CreateSpace(XrSession InSession)
{
	if (Action == XR_NULL_HANDLE || Space != XR_NULL_HANDLE)
	{
		return false;
	}

	XrActionSpaceCreateInfo ActionSpaceInfo;
	ActionSpaceInfo.type = XR_TYPE_ACTION_SPACE_CREATE_INFO;
	ActionSpaceInfo.next = nullptr;
	ActionSpaceInfo.subactionPath = SubactionPath;
	ActionSpaceInfo.poseInActionSpace = ToXrPose(FTransform::Identity);
	ActionSpaceInfo.action = Action;
	return XR_ENSURE(xrCreateActionSpace(InSession, &ActionSpaceInfo, &Space));
}

void FOpenXRHMD::FDeviceSpace::DestroySpace()
{
	if (Space)
	{
		XR_ENSURE(xrDestroySpace(Space));
	}
	Space = XR_NULL_HANDLE;
}

//---------------------------------------------------
// OpenXR Tracking Space Implementation
//---------------------------------------------------

FOpenXRHMD::FTrackingSpace::FTrackingSpace(XrReferenceSpaceType InType)
	: FTrackingSpace(InType, ToXrPose(FTransform::Identity))
{
}

FOpenXRHMD::FTrackingSpace::FTrackingSpace(XrReferenceSpaceType InType, XrPosef InBasePose)
	: Type(InType)
	, Handle(XR_NULL_HANDLE)
	, BasePose(InBasePose)
{
}

FOpenXRHMD::FTrackingSpace::~FTrackingSpace()
{
	DestroySpace();
}

bool FOpenXRHMD::FTrackingSpace::CreateSpace(XrSession InSession)
{
	DestroySpace();

	XrReferenceSpaceCreateInfo SpaceInfo;
	SpaceInfo.type = XR_TYPE_REFERENCE_SPACE_CREATE_INFO;
	SpaceInfo.next = nullptr;
	SpaceInfo.referenceSpaceType = Type;
	SpaceInfo.poseInReferenceSpace = BasePose;
	return XR_ENSURE(xrCreateReferenceSpace(InSession, &SpaceInfo, &Handle));
}

void FOpenXRHMD::FTrackingSpace::DestroySpace()
{
	if (Handle)
	{
		XR_ENSURE(xrDestroySpace(Handle));
	}
	Handle = XR_NULL_HANDLE;
}

XrPath FOpenXRHMD::GetUserPathForControllerHand(EControllerHand Hand)
{
	XrPath Path = XR_NULL_PATH;
	switch  (Hand)
	{
	case EControllerHand::Left:
		Path = FOpenXRPath("/user/hand/left");
		break;
	case EControllerHand::Right:
		Path = FOpenXRPath("/user/hand/right");
		break;
	case EControllerHand::HMD:
		Path = FOpenXRPath("/user/head");
		break;
	default:
		break;
	}
	
	return Path;
}

#undef LOCTEXT_NAMESPACE
