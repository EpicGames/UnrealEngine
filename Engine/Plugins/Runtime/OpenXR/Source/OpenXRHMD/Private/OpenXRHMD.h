// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenXRHMD_Layer.h"
#include "OpenXRAssetManager.h"
#include "CoreMinimal.h"
#include "HeadMountedDisplayBase.h"
#include "XRTrackingSystemBase.h"
#include "XRRenderTargetManager.h"
#include "XRRenderBridge.h"
#include "XRSwapChain.h"
#include "SceneViewExtension.h"
#include "DefaultSpectatorScreenController.h"
#include "IHeadMountedDisplayVulkanExtensions.h"
#include "IOpenXRExtensionPluginDelegates.h"
#include "IOpenXRHMD.h"
#include "Misc/EnumClassFlags.h"
#include "XRCopyTexture.h"
#include "Slate/SlateViewportProvider.h"

#include <openxr/openxr.h>

#include "DefaultStereoLayers.h"

class APlayerController;
class FSceneView;
class FSceneViewFamily;
class FFBFoveationImageGenerator;
class FOpenXRSwapchain;
class UCanvas;
class FOpenXRRenderBridge;
class IOpenXRInputModule;
struct FDefaultStereoLayers_LayerRenderParams;
union FXrCompositionLayerUnion;

/**
 * Simple Head Mounted Display
 */
class FOpenXRHMD
	: public FHeadMountedDisplayBase
	, public FXRRenderTargetManager
	, public FHMDSceneViewExtension
	, public FOpenXRAssetManager
	, public FSimpleLayerManager
	, public IOpenXRExtensionPluginDelegates
	, public IOpenXRHMD
{
private:

public:
	class FDeviceSpace
	{
	public:
		FDeviceSpace(XrAction InAction, XrPath InPath);
		FDeviceSpace(XrAction InAction, XrPath InPath, XrPath InSubactionPath);
		~FDeviceSpace();

		bool CreateSpace(XrSession InSession);
		void DestroySpace();

		XrAction Action;
		XrSpace Space;
		XrPath Path;
		XrPath SubactionPath;
	};

	class FTrackingSpace
	{
	public:
		FTrackingSpace(XrReferenceSpaceType InType);
		FTrackingSpace(XrReferenceSpaceType InType, XrPosef InBasePose);
		~FTrackingSpace();

		bool CreateSpace(XrSession InSession);
		void DestroySpace();

		XrReferenceSpaceType Type;
		XrSpace Handle;
		XrPosef BasePose;
	};

	// The game and render threads each have a separate copy of these structures so that they don't stomp on each other or cause tearing
	// when the game thread progresses to the next frame while the render thread is still working on the previous frame.
	struct FPipelinedFrameState
	{
		XrFrameState FrameState{XR_TYPE_FRAME_STATE};
		XrViewState ViewState{XR_TYPE_VIEW_STATE};
		TArray<XrView> Views;
		TArray<XrViewConfigurationView> ViewConfigs;
		TArray<XrSpaceLocation> DeviceLocations;
		TSharedPtr<FTrackingSpace> TrackingSpace;
		float WorldToMetersScale = 100.0f;
		float PixelDensity = 1.0f;
		int WaitCount = 0;
		int BeginCount = 0;
		int EndCount = 0;
		bool bXrFrameStateUpdated = false;
	};

	struct FPipelinedFrameStateAccessorReadOnly
	{
		FPipelinedFrameStateAccessorReadOnly(FPipelinedFrameState const& InPipelinedFrameState, FRWLock const& InAccessGuard)
			: PipelinedFrameState(InPipelinedFrameState), AccessGuard(const_cast<FRWLock&>(InAccessGuard))
		{
			AccessGuard.ReadLock();
		}

		// not virtual because we aren't going to delete via a base pointer
		~FPipelinedFrameStateAccessorReadOnly()
		{
			AccessGuard.ReadUnlock();
		}

		FPipelinedFrameState const& GetFrameState() const
		{
			return PipelinedFrameState;
		}

	private:

		/** The frame state we're guarding the access to. */
		FPipelinedFrameState const& PipelinedFrameState;

		/** Reference to the guarding lock. */
		FRWLock& AccessGuard;
	};

	struct FPipelinedFrameStateAccessorReadWrite
	{
		FPipelinedFrameStateAccessorReadWrite(FPipelinedFrameState& InPipelinedFrameState, FRWLock& InAccessGuard)
			: PipelinedFrameState(InPipelinedFrameState), AccessGuard(InAccessGuard)
		{
			AccessGuard.WriteLock();
		}

		// not virtual because we aren't going to delete via a base pointer
		~FPipelinedFrameStateAccessorReadWrite()
		{
			AccessGuard.WriteUnlock();
		}

		FPipelinedFrameState& GetFrameState()
		{
			return PipelinedFrameState;
		}

	private:

		/** The frame state we're guarding the access to. */
		FPipelinedFrameState& PipelinedFrameState;

		/** Reference to the guarding lock. */
		FRWLock& AccessGuard;
	};

	struct FEmulatedLayerState
	{
		// These layers are used as a target to composite all the emulated face locked layers into
		// and be sent to the compositor with VIEW tracking space to avoid reprojection.
		TArray<XrCompositionLayerProjectionView> CompositedProjectionLayers;
		TArray<XrSwapchainSubImage> EmulationImages;
		// This swapchain is where the emulated face locked layers are rendered into.
		FXRSwapChainPtr EmulationSwapchain;
	};

	struct FBasePassLayerBlendParameters
	{
		// Default constructor inverts the alpha for color blending to make up for the fact that UE uses
		// alpha = 0 for opaque and alpha = 1 for transparent while OpenXR does the opposite.
		// Alpha blending passes through the destination alpha instead.
		FBasePassLayerBlendParameters()
		{
			srcFactorColor = XR_BLEND_FACTOR_ONE_MINUS_SRC_ALPHA_FB;
			dstFactorColor = XR_BLEND_FACTOR_SRC_ALPHA_FB;
			srcFactorAlpha = XR_BLEND_FACTOR_ZERO_FB;
			dstFactorAlpha = XR_BLEND_FACTOR_ONE_FB;
		}
		
		XrBlendFactorFB	srcFactorColor;
		XrBlendFactorFB	dstFactorColor;
		XrBlendFactorFB	srcFactorAlpha;
		XrBlendFactorFB	dstFactorAlpha;
	};

	struct FLayerColorScaleAndBias
	{
		// Used by XR_KHR_composition_layer_color_scale_bias to apply a color multiplier and offset to the background layer
		// and set via UHeadMountedDisplayFunctionLibrary::SetHMDColorScaleAndBias() --> OpenXRHMD::SetColorScaleAndBias()
		XrColor4f ColorScale;
		XrColor4f ColorBias;
	};

	enum class EOpenXRLayerStateFlags : uint32
	{
		None = 0u,
		BackgroundLayerVisible = (1u << 0),
		SubmitBackgroundLayer = (1u << 1),
		SubmitDepthLayer = (1u << 2),
		SubmitEmulatedFaceLockedLayer = (1u << 3),
		SubmitMotionVectorLayer = (1u << 4),
	};
	FRIEND_ENUM_CLASS_FLAGS(EOpenXRLayerStateFlags);

	struct FPipelinedLayerState
	{
		TArray<FXrCompositionLayerUnion> NativeOverlays;
		TArray<FXrCompositionLayerUnion> NativeUnderlays; // Underlays will be rendered below the projection layers
		TArray<XrCompositionLayerProjectionView> ProjectionLayers;
		TArray<XrCompositionLayerDepthInfoKHR> DepthLayers;
		TArray<XrCompositionLayerDepthTestFB> CompositionDepthTestLayers;

		TArray<XrFrameSynthesisInfoEXT> FrameSynthesisLayers;
		TArray<XrCompositionLayerSpaceWarpInfoFB> SpaceWarpLayers;

		TArray<XrSwapchainSubImage> ColorImages;
		TArray<XrSwapchainSubImage> DepthImages;

		TArray<XrSwapchainSubImage> MotionVectorImages;
		TArray<XrSwapchainSubImage> MotionVectorDepthImages;

		FXRSwapChainPtr ColorSwapchain;
		FXRSwapChainPtr DepthSwapchain;
		FXRSwapChainPtr MotionVectorSwapchain;
		FXRSwapChainPtr MotionVectorDepthSwapchain;
		TArray<FXRSwapChainPtr> NativeLayerSwapchains;	// This is currently only used as a reference to keep the swapchains from being destroyed

		FEmulatedLayerState EmulatedLayerState;

		EOpenXRLayerStateFlags LayerStateFlags = EOpenXRLayerStateFlags::None;
		FBasePassLayerBlendParameters BasePassLayerBlendParams;
		FLayerColorScaleAndBias LayerColorScaleAndBias;
	};

	class FVulkanExtensions : public IHeadMountedDisplayVulkanExtensions
	{
	public:
		FVulkanExtensions(XrInstance InInstance, XrSystemId InSystem) : Instance(InInstance), System(InSystem) {}
		virtual ~FVulkanExtensions() {}

		/** IHeadMountedDisplayVulkanExtensions */
		virtual bool GetVulkanInstanceExtensionsRequired(TArray<const ANSICHAR*>& Out) override;
		virtual bool GetVulkanDeviceExtensionsRequired(VkPhysicalDevice_T *pPhysicalDevice, TArray<const ANSICHAR*>& Out) override;

	private:
		XrInstance Instance;
		XrSystemId System;

		TArray<char> Extensions;
		TArray<char> DeviceExtensions;
	};

	/** IXRTrackingSystem interface */
	virtual FName GetSystemName() const override
	{
		// This identifier is relied upon for plugin identification,
		// see GetHMDName() to query the true XR system name.
		static FName DefaultName(TEXT("OpenXR"));
		return DefaultName;
	}

	int32 GetXRSystemFlags() const override
	{
		int32 flags = EXRSystemFlags::IsHeadMounted;

		if (SelectedEnvironmentBlendMode != XR_ENVIRONMENT_BLEND_MODE_OPAQUE)
		{
			flags |= EXRSystemFlags::IsAR;
		}

		if (bSupportsHandTracking)
		{
			flags |= EXRSystemFlags::SupportsHandTracking;
		}

		return flags;
	}

	virtual bool EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type = EXRTrackedDeviceType::Any) override;

	virtual void SetInterpupillaryDistance(float NewInterpupillaryDistance) override;
	virtual float GetInterpupillaryDistance() const override;
	virtual bool GetRelativeEyePose(int32 InDeviceId, int32 InViewIndex, FQuat& OutOrientation, FVector& OutPosition) override;

	virtual void ResetOrientationAndPosition(float Yaw = 0.f) override;
	virtual void ResetOrientation(float Yaw = 0.f) override;
	virtual void ResetPosition() override;
	virtual void Recenter(EOrientPositionSelector::Type Selector, float Yaw = 0.f);

	virtual bool GetIsTracked(int32 DeviceId) override;
	virtual bool GetCurrentPose(int32 DeviceId, FQuat& CurrentOrientation, FVector& CurrentPosition) override;
	virtual bool GetPoseForTime(int32 DeviceId, FTimespan Timespan, bool& OutTimeWasUsed, FQuat& CurrentOrientation, FVector& CurrentPosition, bool& bProvidedLinearVelocity, FVector& LinearVelocity, bool& bProvidedAngularVelocity, FVector& AngularVelocityAsAxisAndLength, bool& bProvidedLinearAcceleration, FVector& LinearAcceleration, float WorldToMetersScale) override;
	virtual bool GetCurrentInteractionProfile(const EControllerHand Hand, FString& InteractionProfile) override;
	
	virtual void SetBaseRotation(const FRotator& InBaseRotation) override;
	virtual FRotator GetBaseRotation() const override;

	virtual void SetBaseOrientation(const FQuat& InBaseOrientation) override;
	virtual FQuat GetBaseOrientation() const override;

	virtual void SetBasePosition(const FVector& InBasePosition) override;
	virtual FVector GetBasePosition() const override;

	virtual void SetTrackingOrigin(EHMDTrackingOrigin::Type NewOrigin) override;
	virtual EHMDTrackingOrigin::Type GetTrackingOrigin() const override;

	virtual class IHeadMountedDisplay* GetHMDDevice() override
	{
		return this;
	}
	virtual class TSharedPtr< class IStereoRendering, ESPMode::ThreadSafe > GetStereoRenderingDevice() override
	{
		return SharedThis(this);
	}

	virtual class IStereoLayers* GetStereoLayers() override
	{
		return this;
	}

	virtual void GetMotionControllerState(UObject* WorldContext, const EXRSpaceType XRSpaceType, const EControllerHand Hand, const EXRControllerPoseType XRControllerPoseType, FXRMotionControllerState& MotionControllerState) override;
	virtual void GetHandTrackingState(UObject* WorldContext, const EXRSpaceType XRSpaceType, const EControllerHand Hand, FXRHandTrackingState& HandTrackingState) override;

	virtual float GetWorldToMetersScale() const override;

	virtual FVector2D GetPlayAreaBounds(EHMDTrackingOrigin::Type Origin) const override;
	virtual bool GetPlayAreaRect(FTransform& OutTransform, FVector2D& OutExtent) const override;
	virtual bool GetTrackingOriginTransform(TEnumAsByte<EHMDTrackingOrigin::Type> Origin, FTransform& OutTransform)  const override;

	virtual bool HDRGetMetaDataForStereo(EDisplayOutputFormat& OutDisplayOutputFormat, EDisplayColorGamut& OutDisplayColorGamut, bool& OutbHDRSupported) override;

protected:
	bool StartSession();
	bool StopSession();
	bool OnStereoStartup();
	bool OnStereoTeardown();
	bool ReadNextEvent(XrEventDataBuffer* buffer);
	void DestroySession();

	void RequestExitApp();

	void BuildOcclusionMeshes();
	bool BuildOcclusionMesh(XrVisibilityMaskTypeKHR Type, int View, FHMDViewMesh& Mesh);

	FPipelinedFrameStateAccessorReadOnly GetPipelinedFrameStateForThread() const;
	FPipelinedFrameStateAccessorReadWrite GetPipelinedFrameStateForThread();

	void UpdateDeviceLocations(bool bUpdateOpenXRExtensionPlugins);
	void EnumerateViews(FPipelinedFrameState& PipelineState);
	void LocateViews(FPipelinedFrameState& PipelinedState, bool ResizeViewsArray = false);

	void AllocateDepthTextureInternal(uint32 SizeX, uint32 SizeY, uint32 NumSamples, uint32 ArraySize);

	void SetupFrameLayers_GameThread();
	void SetupFrameLayers_RenderThread(FRDGBuilder& GraphBuilder);
	void DrawEmulatedLayers_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView);
	void DrawBackgroundCompositedEmulatedLayers_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView);
	void DrawEmulatedFaceLockedLayers_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView);
	void DrawAllEmulatedLayers_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView);

	/** TStereoLayerManager<FLayerDesc> */
	virtual void MarkTextureForUpdate(uint32 LayerId) override;
	virtual uint32 CreateLayer(const FLayerDesc& InLayerDesc) override;
	virtual void DestroyLayer(uint32 LayerId) override;
	virtual void SetLayerDesc(uint32 LayerId, const FLayerDesc& InLayerDesc) override;

	virtual bool PopulateAnalyticsAttributes(TArray<struct FAnalyticsEventAttribute>& EventAttributes) override;

	/** Populates System, SystemProperties and related fields. Can get called before OnStereoStartup. */
	bool AcquireSystemIdAndProperties();
	
	[[nodiscard]] int32 GetNumHeaders() const;

public:
	/** IXRTrackingSystem interface */
	virtual bool DoesSupportLateProjectionUpdate() const override { return true; }
	virtual FString GetVersionString() const override;
	virtual bool IsTracking(int32 DeviceId) override;
	virtual bool HasValidTrackingPosition() override;
	virtual IOpenXRHMD* GetIOpenXRHMD() { return this; }

	/** IHeadMountedDisplay interface */
	virtual bool IsHMDConnected() override;
	virtual bool DoesSupportPositionalTracking() const override { return true; }
	virtual bool IsHMDEnabled() const override;
	virtual void EnableHMD(bool allow = true) override;
	virtual FName GetHMDName() const override;
	virtual bool GetHMDMonitorInfo(MonitorInfo&) override;
	virtual void GetFieldOfView(float& OutHFOVInDegrees, float& OutVFOVInDegrees) const override;
	virtual bool IsChromaAbCorrectionEnabled() const override;
	virtual float GetPixelDenity() const override;
	virtual void SetPixelDensity(const float NewDensity) override;
	virtual FIntPoint GetIdealRenderTargetSize() const override;
	virtual bool GetHMDDistortionEnabled(EShadingPath ShadingPath) const override { return false; }
	virtual FIntRect GetFullFlatEyeRect_RenderThread(const FRHITextureDesc& EyeTexture) const override;
	virtual bool HasHiddenAreaMesh() const override final;
	virtual bool HasVisibleAreaMesh() const override final;
	virtual void DrawHiddenAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex) const override final;
	virtual void DrawVisibleAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex) const override final;
	virtual void DrawHiddenAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex, int32 InstanceCount) const override final;
	virtual void DrawVisibleAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex, int32 InstanceCount) const override final;
	virtual void OnBeginRendering_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& ViewFamily) override;
	virtual void OnBeginRendering_GameThread(FSceneViewFamily& SceneViewFamily) override;
	virtual void OnLateUpdateApplied_RenderThread(FRDGBuilder& GraphBuilder, const FTransform& NewRelativeTransform) override;
	virtual bool OnStartGameFrame(FWorldContext& WorldContext) override;
	virtual EHMDWornState::Type GetHMDWornState() override { return bIsReady ? EHMDWornState::Worn : EHMDWornState::NotWorn; }
	virtual bool SetColorScaleAndBias(FLinearColor ColorScale, FLinearColor ColorBias);

	/** IStereoRendering interface */
	virtual bool IsStereoEnabled() const override;
	virtual bool EnableStereo(bool stereo = true) override;
	virtual void AdjustViewRect(int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const override;
	virtual void SetFinalViewRect(FRHICommandListImmediate& RHICmdList, const int32 StereoViewIndex, const FIntRect& FinalViewRect) override;
	virtual int32 GetDesiredNumberOfViews(bool bStereoRequested) const override;
	virtual EStereoscopicPass GetViewPassForIndex(bool bStereoRequested, int32 ViewIndex) const override;
	virtual uint32 GetLODViewIndex() const override;
	virtual bool IsStandaloneStereoOnlyDevice() const override { return bIsStandaloneStereoOnlyDevice; }

	virtual FMatrix GetStereoProjectionMatrix(const int32 StereoViewIndex) const override;
	virtual void GetEyeRenderParams_RenderThread(const struct FHeadMountedDisplayPassContext& Context, FVector2D& EyeToSrcUVScaleValue, FVector2D& EyeToSrcUVOffsetValue) const override;
	virtual IStereoRenderTargetManager* GetRenderTargetManager() override;

	virtual void RenderTexture_RenderThread(class FRDGBuilder& GraphBuilder, FRDGTextureRef BackBuffer, FRDGTextureRef SrcTexture, FVector2f WindowSize) const override;

	/** ISceneViewExtension interface */
	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual void PostRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override; // for non-face locked compositing
	virtual void PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;

	/** FHMDSceneViewExtension interface */
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;

	/** IStereoRenderTargetManager */
	virtual bool ShouldUseSeparateRenderTarget() const override { return IsStereoEnabled() && RenderBridge.IsValid(); }
	virtual void CalculateRenderTargetSize(const FViewport& Viewport, uint32& InOutSizeX, uint32& InOutSizeY) override;
	virtual bool AllocateRenderTargetTextures(FRHICommandListBase& RHICmdList, uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumLayers, ETextureCreateFlags Flags, ETextureCreateFlags TargetableTextureFlags, TArray<FTextureRHIRef>& OutTargetableTextures, TArray<FTextureRHIRef>& OutShaderResourceTextures, uint32 NumSamples = 1) override;
	virtual int32 AcquireColorTexture() override final;
	virtual int32 AcquireDepthTexture() override final;
	virtual bool AllocateDepthTexture(uint32 Index, uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags InTexFlags, ETextureCreateFlags TargetableTextureFlags, FTextureRHIRef& OutTargetableTexture, FTextureRHIRef& OutShaderResourceTexture, uint32 NumSamples = 1) override final;
	virtual bool GetRecommendedMotionVectorTextureSize(FIntPoint& OutTextureSize) override;
	virtual bool GetMotionVectorTexture(uint32 Index, const FIntPoint& Size, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, FTextureRHIRef& OutTexture, uint32 NumSamples = 1) override;
	virtual bool GetMotionVectorDepthTexture(uint32 Index, const FIntPoint& Size, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, FTextureRHIRef& OutTexture, uint32 NumSamples = 1) override;
	virtual bool ReconfigureForShaderPlatform(EShaderPlatform NewShaderPlatform) override;
	virtual EPixelFormat GetActualColorSwapchainFormat() const override { return static_cast<EPixelFormat>(LastActualColorSwapchainFormat); }

	/** IXRTrackingSystem */
	virtual void OnBeginPlay(FWorldContext& InWorldContext) override;

	/** IStereoLayers */
	virtual TArray<FTextureRHIRef, TInlineAllocator<2>> GetDebugLayerTextures_RenderThread() override;

	/** IOpenXRExtensionPluginDelegates */
public:
	virtual FApplyHapticFeedbackAddChainStructsDelegate& GetApplyHapticFeedbackAddChainStructsDelegate() override { return ApplyHapticFeedbackAddChainStructsDelegate; }
private:
	FApplyHapticFeedbackAddChainStructsDelegate ApplyHapticFeedbackAddChainStructsDelegate;

public:
	/** Constructor */
	FOpenXRHMD(const FAutoRegister&, XrInstance InInstance, TRefCountPtr<FOpenXRRenderBridge>& InRenderBridge, TArray<const char*> InEnabledExtensions, TArray<class IOpenXRExtensionPlugin*> InExtensionPlugins, IARSystemSupport* ARSystemSupport, EOpenXRAPIVersion InOpenXRAPIVersion);


	/** Destructor */
	virtual ~FOpenXRHMD();

	void OnBeginSimulation_GameThread();
	void OnBeginRendering_RHIThread(IRHICommandContext& RHICmdContext, const FPipelinedFrameState& InFrameState, FXRSwapChainPtr ColorSwapchain, FXRSwapChainPtr DepthSwapchain, FXRSwapChainPtr MotionVectorSwapchain, FXRSwapChainPtr MotionVectorDepthSwapchain, FXRSwapChainPtr EmulationSwapchain);
	void OnFinishRendering(IRHICommandContext* RHICmdContext);
	void OnBackBufferReadyToPresent(class SWindow& Window, ISlateViewportProvider& ViewportProvider);

	/** IOpenXRHMD */
	void SetInputModule(IOpenXRInputModule* InInputModule) override
	{
		InputModule = InInputModule;
	}
	/** @return	True if the HMD was initialized OK */
	bool IsInitialized() const override;
	bool IsRunning() const override;
	bool IsFocused() const override;
	int32 AddTrackedDevice(XrAction Action, XrPath Path) override;
	int32 AddTrackedDevice(XrAction Action, XrPath Path, XrPath SubActionPath) override;
	void ResetTrackedDevices() override;
	XrPath GetTrackedDevicePath(const int32 DeviceId) override;
	XrSpace GetTrackedDeviceSpace(const int32 DeviceId) override;
	bool IsExtensionEnabled(const FString& Name) const override { return EnabledExtensions.Contains(Name); }
	bool IsOpenXRAPIVersionMet(EOpenXRAPIVersion RequiredVersion) const override { return OpenXRAPIVersion >= RequiredVersion; }

	XrInstance GetInstance() override { return Instance; }
	XrSystemId GetSystem() override { return System; }
	XrSession GetSession() override { return Session; }
	XrTime GetDisplayTime() const override;
	XrSpace GetTrackingSpace() const override;
	IOpenXRExtensionPluginDelegates& GetIOpenXRExtensionPluginDelegates() override { return *this; }
	TArray<IOpenXRExtensionPlugin*>& GetExtensionPlugins() override { return ExtensionPlugins; }
	
	void SetEnvironmentBlendMode(XrEnvironmentBlendMode NewBlendMode) override;
	XrEnvironmentBlendMode GetEnvironmentBlendMode() override { return SelectedEnvironmentBlendMode; }
	TArray<XrEnvironmentBlendMode> GetSupportedEnvironmentBlendModes() const override;

	/** Provides the full asymmetric FOV information for the view, which allows proper centering of screen-based techniques based on forward eye gaze */
	void GetFullFOVInformation(TArray<XrFovf>& FovInfos) const override;

	virtual bool AllocateSwapchainTextures_RenderThread(const FOpenXRSwapchainProperties& InSwapchainProperies, FXRSwapChainPtr& InOutSwapchain, uint8& OutActualFormat) override;

	/** Returns shader platform the plugin is currently configured for, in the editor it can change due to preview platforms. */
	EShaderPlatform GetConfiguredShaderPlatform() const { check(ConfiguredShaderPlatform != EShaderPlatform::SP_NumPlatforms); return ConfiguredShaderPlatform; }
	FOpenXRSwapchain* GetColorSwapchain_RenderThread();

	bool RuntimeRequiresRHIContext() const { return bRuntimeRequiresRHIContext; }
	
private:
	FDefaultStereoLayers_LayerRenderParams CalculateEmulatedLayerRenderParams(const FSceneView& InView);
	
	class FEmulatedLayersPass* SetupEmulatedLayersRenderPass(
		FRDGBuilder& GraphBuilder,
		const FSceneView& InView,
		TArray<FDefaultStereoLayers::FStereoLayerToRender>& Layers,
		FRDGTextureRef RenderTarget,
		FDefaultStereoLayers_LayerRenderParams& OutRenderParams);
	
	[[nodiscard]] FEmulatedLayersPass* SetupEmulatedLayerRenderPass(
		FRDGBuilder& GraphBuilder,
		const FSceneView& InView,
		const FDefaultStereoLayers::FStereoLayerToRender& Layer,
		FRDGTextureRef RenderTarget,
		FRDGTextureRef DepthTarget,
		FDefaultStereoLayers_LayerRenderParams& OutRenderParams);
	
	bool IsEmulatingStereoLayers();
	void CopySwapchainTexture_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef SrcTexture, FIntRect SrcRect, const FXRSwapChainPtr& DstSwapChain, FIntRect DstRect,
								  bool bClearBlack, EXRCopyTextureBlendModifier SrcTextureCopyModifier, FStaticFeatureLevel FeatureLevel, FStaticShaderPlatform ShaderPlatform) const;

	void UpdateLayerSwapchainTexture(FOpenXRLayer& Layer, FRDGBuilder& GraphBuilder);
	void ConfigureLayerSwapchains(const struct FLayerToUpdateSwapchain& Update, FOpenXRLayer& Layer);

	// Gets the user path corresponding to the input EControllerHand value
	static XrPath GetUserPathForControllerHand(EControllerHand Hand);

	bool					bStereoEnabled;
	TAtomic<bool>			bIsRunning;
	TAtomic<bool>			bIsReady;
	TAtomic<bool>			bIsRendering;
	TAtomic<bool>			bIsSynchronized;
	bool					bShouldWait;
	bool					bIsExitingSessionByxrRequestExitSession;
	bool					bDepthExtensionSupported;
	bool					bHiddenAreaMaskSupported;
	bool					bViewConfigurationFovSupported;
	bool					bNeedReBuildOcclusionMesh;
	bool					bIsMobileMultiViewEnabled;
	bool					bSupportsHandTracking;
	bool					bSpaceAccelerationSupported;
	bool					bProjectionLayerAlphaEnabled;
	bool					bIsStandaloneStereoOnlyDevice;
	bool					bRuntimeRequiresRHIContext;
	bool					bIsTrackingOnlySession;
	bool					bIsAcquireOnAnyThreadSupported;
	bool					bUseWaitCountToAvoidExtraXrBeginFrameCalls;
	bool					bEquirectLayersSupported;
	bool					bCylinderLayersSupported;
	bool					bCubemapLayersSupported;
	float					WorldToMetersScale = 100.0f;
	float					RuntimePixelDensityMax = FHeadMountedDisplayBase::PixelDensityMax;
	EShaderPlatform			ConfiguredShaderPlatform = EShaderPlatform::SP_NumPlatforms;

	XrSessionState			CurrentSessionState;
	FRWLock					SessionHandleMutex;

	TArray<const char*>		EnabledExtensions;
	IOpenXRInputModule*		InputModule;
	TArray<class IOpenXRExtensionPlugin*> ExtensionPlugins;
	XrInstance				Instance;
	EOpenXRAPIVersion		OpenXRAPIVersion;
	XrSystemId				System;
	XrSession				Session;
	XrSpace					LocalSpace;
	XrSpace					LocalFloorSpace;
	XrSpace					StageSpace;
	XrSpace					CustomSpace;
	XrReferenceSpaceType	TrackingSpaceType;
	XrViewConfigurationType SelectedViewConfigurationType;
	XrEnvironmentBlendMode  SelectedEnvironmentBlendMode;
	XrInstanceProperties    InstanceProperties;
	XrSystemProperties      SystemProperties;

	FPipelinedFrameState	PipelinedFrameStateGame;
	FPipelinedFrameState	PipelinedFrameStateRendering;
	FPipelinedFrameState	PipelinedFrameStateRHI;

	/** Arbitrates access to PipelinedFrameStateGame from code on task threads */
	FRWLock					PipelinedFrameStateGameAccessGuard;
	/** Arbitrates access to PipelinedFrameStateRendering from code on task threads */
	FRWLock					PipelinedFrameStateRenderingAccessGuard;
	// we do not expose PipelineFrameStateRHI in GetPipelinedFrameStateForThread() as of now, so no lock is provided for that one.

	FPipelinedLayerState	PipelinedLayerStateRendering;
	FPipelinedLayerState	PipelinedLayerStateRHI;

	mutable FRWLock			DeviceMutex;
	TArray<FDeviceSpace>	DeviceSpaces;

	TRefCountPtr<FOpenXRRenderBridge> RenderBridge;
	IRendererModule*		RendererModule;

	uint8					LastRequestedColorSwapchainFormat;
	uint8					LastActualColorSwapchainFormat;
	uint8					LastRequestedDepthSwapchainFormat;

	TArray<FHMDViewMesh>	HiddenAreaMeshes;
	TArray<FHMDViewMesh>	VisibleAreaMeshes;

	bool					bTrackingSpaceInvalid;
	bool					bUseCustomReferenceSpace;
	FQuat					BaseOrientation;
	FVector					BasePosition;

	bool					bLayerSupportOpenXRCompliant;
	bool					bOpenXRForceStereoLayersEmulationCVarCachedValue;
	TArray<uint32>			VisibleLayerIds;
	TArray<FDefaultStereoLayers::FStereoLayerToRender> BackgroundCompositedEmulatedLayers;
	TArray<FDefaultStereoLayers::FStereoLayerToRender> EmulatedFaceLockedLayers;
	TArray<FDefaultStereoLayers::FStereoLayerToRender> EmulatedLayers;
	TArray<FOpenXRLayer>	NativeLayers;

	TUniquePtr<FFBFoveationImageGenerator> FBFoveationImageGenerator;
	bool					bFoveationExtensionSupported;
	bool					bRuntimeFoveationSupported;
	bool					bLocalFloorSpaceSupported;
	bool					bUserPresenceSupported;

	XrColor4f				LayerColorScale;
	XrColor4f				LayerColorBias;
	bool					bCompositionLayerColorScaleBiasSupported;
	bool					bxrGetSystemPropertiesSuccessful;
	
	FTransform				FrameSynthesisLastTrackingToWorld;
	FIntPoint				RecommendedMotionVectorTextureSize;
};

ENUM_CLASS_FLAGS(FOpenXRHMD::EOpenXRLayerStateFlags);
