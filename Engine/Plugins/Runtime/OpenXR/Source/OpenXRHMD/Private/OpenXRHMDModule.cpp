// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRHMDModule.h"
#include "Misc/EngineVersion.h"
#include "Misc/ConfigUtilities.h"
#include "OpenXRHMD.h"
#include "OpenXRHMD_RenderBridge.h"
#include "OpenXRCore.h"
#include "IOpenXRExtensionPlugin.h"
#include "IOpenXRARModule.h"
#include "OpenXRHMDSettings.h"
#include "BuildSettings.h"
#include "GeneralProjectSettings.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Epic_openxr.h"

#if PLATFORM_ANDROID
#include <android_native_app_glue.h>
extern struct android_app* GNativeAndroidApp;

extern bool AndroidThunkCpp_IsOculusMobileApplication();
extern bool AndroidThunkCpp_IsXRImmersiveHMDApplication();
#endif

static TAutoConsoleVariable<int32> CVarEnableOpenXRValidationLayer(
	TEXT("xr.EnableOpenXRValidationLayer"),
	0,
	TEXT("If true, enables the OpenXR validation layer, which will provide extended validation of\nOpenXR API calls. This should only be used for debugging purposes.\n")
	TEXT("Changes will only take effect in new game/editor instances - can't be changed at runtime.\n"),
	ECVF_Default);		// @todo: Should we specify ECVF_Cheat here so this doesn't show up in release builds?

static TAutoConsoleVariable<bool> CVarCheckOpenXRInstanceConformance(
	TEXT("xr.CheckOpenXRInstanceConformance"),
	true,
	TEXT("If true, OpenXR will verify Instance is conformant by calling xrStringToPath. Some runtimes fail without a system attached at instance creation time."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarRetainPreInitInstance(
	TEXT("xr.RetainPreInitInstance"),
	false,
	TEXT("If true, OpenXR will retain any instance created during PreInit rather than destroying it.  Destroying it is more correct because we are not yet certain to have chosen OpenXRHMD, and another HMD plugin could take over and try to create an instance of its own which would fail on a runtime that supports only one."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarPreferFBSpaceWarp(
	TEXT("xr.PreferFBSpaceWarp"),
	false,
	TEXT("If true, OpenXR will use XR_FB_space_warp rather than XR_EXT_frame_synthesis to provide frame synthesis when both extensions are available."),
	ECVF_ReadOnly);

//---------------------------------------------------
// OpenXRHMD Plugin Implementation
//---------------------------------------------------

IMPLEMENT_MODULE( FOpenXRHMDModule, OpenXRHMD )

FOpenXRHMDModule::FOpenXRHMDModule()
	: LoaderHandle(nullptr)
	, Instance(XR_NULL_HANDLE)
	, InstanceOpenXRAPIVersion(EOpenXRAPIVersion::V_INVALID)
	, RenderBridge(nullptr)
	, OculusAudioInputDevice()
	, OculusAudioOutputDevice()
{ }

FOpenXRHMDModule::~FOpenXRHMDModule()
{
}

TSharedPtr< class IXRTrackingSystem, ESPMode::ThreadSafe > FOpenXRHMDModule::CreateTrackingSystem()
{
	if (!InitInstance())
	{
		return nullptr;
	}

	if (!RenderBridge && !FParse::Param(FCommandLine::Get(), TEXT("xrtrackingonly")))
	{
		if (!InitRenderBridge())
		{
			return nullptr;
		}
	}
	auto ARModule = FModuleManager::LoadModulePtr<IOpenXRARModule>("OpenXRAR");
	auto ARSystem = ARModule->CreateARSystem();

	if (!Instance)
	{
		return nullptr;
	}

	auto OpenXRHMD = FSceneViewExtensions::NewExtension<FOpenXRHMD>(Instance, RenderBridge, EnabledExtensions, ExtensionPlugins, ARSystem, InstanceOpenXRAPIVersion);
	if (OpenXRHMD->IsInitialized())
	{
		ARModule->SetTrackingSystem(OpenXRHMD.Get());
		OpenXRHMD->GetARCompositionComponent()->InitializeARSystem();
		return OpenXRHMD;
	}

	return nullptr;
}

bool FOpenXRHMDModule::PreInit()
{
#if PLATFORM_WINDOWS
	// On Windows, we need to get the audio input/output devices before init, so create the instance first, grab the audio devices, and then
	// immediately destroy it so a new one can be created for the actual initialize call
	const bool bInitialized = InitInstance();
	if (bInitialized)
	{
		if (IsExtensionEnabled(XR_OCULUS_AUDIO_DEVICE_GUID_EXTENSION_NAME))
		{
			PFN_xrGetAudioInputDeviceGuidOculus GetAudioInputDeviceGuidOculus = nullptr;
			if (XR_ENSURE(xrGetInstanceProcAddr(Instance, "xrGetAudioInputDeviceGuidOculus", (PFN_xrVoidFunction*)&GetAudioInputDeviceGuidOculus)))
			{
				WCHAR DeviceGuid[XR_MAX_AUDIO_DEVICE_STR_SIZE_OCULUS];
				if (XR_SUCCEEDED(GetAudioInputDeviceGuidOculus(Instance, DeviceGuid)))
				{
					OculusAudioInputDevice = FString(XR_MAX_AUDIO_DEVICE_STR_SIZE_OCULUS, DeviceGuid);
					OculusAudioInputDevice.TrimToNullTerminator();
				}
			}

			PFN_xrGetAudioOutputDeviceGuidOculus GetAudioOutputDeviceGuidOculus = nullptr;
			if (XR_ENSURE(xrGetInstanceProcAddr(Instance, "xrGetAudioOutputDeviceGuidOculus", (PFN_xrVoidFunction*)&GetAudioOutputDeviceGuidOculus)))
			{
				WCHAR DeviceGuid[XR_MAX_AUDIO_DEVICE_STR_SIZE_OCULUS];
				if (XR_SUCCEEDED(GetAudioOutputDeviceGuidOculus(Instance, DeviceGuid)))
				{
					OculusAudioOutputDevice = FString::ConstructFromPtrSize(DeviceGuid, XR_MAX_AUDIO_DEVICE_STR_SIZE_OCULUS);
					OculusAudioOutputDevice.TrimToNullTerminator();
				}
			}
		}
		if (!CVarRetainPreInitInstance.GetValueOnAnyThread())
		{
			XR_ENSURE(xrDestroyInstance(Instance));
			Instance = nullptr;
		}
	}
	return bInitialized;
#else
	return true;
#endif // PLATFORM_WINDOWS
}

void FOpenXRHMDModule::StartupModule()
{
	IOpenXRHMDModule::StartupModule();

	// Pull in CVar settings from OpenXRHMDSettings in BaseEngine.ini
	UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/OpenXRHMD.OpenXRHMDSettings"), *GEngineIni, ECVF_SetByProjectSetting);
}

void FOpenXRHMDModule::ShutdownModule()
{
	if (Instance)
	{
		XR_ENSURE(xrDestroyInstance(Instance));
	}

	if (LoaderHandle)
	{
		FPlatformProcess::FreeDllHandle(LoaderHandle);
		LoaderHandle = nullptr;
	}

	IOpenXRHMDModule::ShutdownModule();
}

uint64 FOpenXRHMDModule::GetGraphicsAdapterLuid()
{
    uint64 DefaultValue = 0;
    
    // Mac platforms expect the device ID to be returned here
#if PLATFORM_MAC
    DefaultValue = (uint64)-1;
#endif
    
	if (FParse::Param(FCommandLine::Get(), TEXT("xrtrackingonly")))
	{
        return DefaultValue;
	}

	if (!RenderBridge)
	{
		if (!InitRenderBridge())
		{
            return DefaultValue;
		}
	}

	FConfigFile* EngineIni = GConfig->FindConfigFile(GEngineIni);
	XrSystemId System = GetSystemId();
	if (!System)
	{
		int64 AdapterLuid = (int64)DefaultValue;
		EngineIni->GetInt64(TEXT("OpenXR.Settings"), TEXT("GraphicsAdapter"), AdapterLuid);
		return reinterpret_cast<uint64&>(AdapterLuid);
	}

	uint64 AdapterLuid = RenderBridge->GetGraphicsAdapterLuid(System);
	if (AdapterLuid)
	{
		// Remember this luid so we use the right adapter, even when we startup without an HMD connected
		EngineIni->SetInt64(TEXT("OpenXR.Settings"), TEXT("GraphicsAdapter"), reinterpret_cast<int64&>(AdapterLuid));
	}
	return AdapterLuid;
}

TSharedPtr< IHeadMountedDisplayVulkanExtensions, ESPMode::ThreadSafe > FOpenXRHMDModule::GetVulkanExtensions()
{
#ifdef XR_USE_GRAPHICS_API_VULKAN
	if (InitInstance() && IsExtensionEnabled(XR_KHR_VULKAN_ENABLE_EXTENSION_NAME))
	{
		XrSystemId System = GetSystemId();
		if (!System)
		{
			return nullptr;
		}

		if (!VulkanExtensions.IsValid())
		{
			VulkanExtensions = MakeShareable(new FOpenXRHMD::FVulkanExtensions(Instance, System));
		}
		return VulkanExtensions;
	}
#endif//XR_USE_GRAPHICS_API_VULKAN
	return nullptr;
}

FString FOpenXRHMDModule::GetDeviceSystemName()
{
	if (InitInstance())
	{
		XrSystemId System = GetSystemId();
		if (System)
		{
			XrSystemProperties SystemProperties;
			SystemProperties.type = XR_TYPE_SYSTEM_PROPERTIES;
			SystemProperties.next = nullptr;
			XR_ENSURE(xrGetSystemProperties(Instance, System, &SystemProperties));

			return FString(UTF8_TO_TCHAR(SystemProperties.systemName)); //-V614
		}
	}
	return FString("");
}

bool FOpenXRHMDModule::IsStandaloneStereoOnlyDevice()
{
	if (InitInstance())
	{
		for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
		{
			if (Module->IsStandaloneStereoOnlyDevice())
			{
				return true;
			}
		}

#if PLATFORM_ANDROID
		return IStereoRendering::IsStartInVR();
#endif
	}
	return false;
}

bool FOpenXRHMDModule::EnumerateExtensions()
{
	uint32_t ExtensionsCount = 0;
	if (XR_FAILED(xrEnumerateInstanceExtensionProperties(nullptr, 0, &ExtensionsCount, nullptr)))
	{
		// If it fails this early that means there's no runtime installed
		UE_LOGF(LogHMD, Log, "xrEnumerateInstanceExtensionProperties failed, suggests no runtime is installed.");
		return false;
	}

	TArray<XrExtensionProperties> Properties;
	Properties.SetNum(ExtensionsCount);
	for (auto& Prop : Properties)
	{
		Prop = XrExtensionProperties{ XR_TYPE_EXTENSION_PROPERTIES };
	}

	if (XR_ENSURE(xrEnumerateInstanceExtensionProperties(nullptr, ExtensionsCount, &ExtensionsCount, Properties.GetData())))
	{
		UE_LOGF(LogHMD, Log, "OpenXR runtime supported extensions:");
		for (const XrExtensionProperties& Prop : Properties)
		{
			AvailableExtensions.Add(Prop.extensionName);
			UE_LOGF(LogHMD, Log, "\t%s", (Prop.extensionName));
		}
		return true;
	}
	return false;
}

bool FOpenXRHMDModule::EnumerateLayers()
{
	uint32 LayerPropertyCount = 0;
	if (XR_FAILED(xrEnumerateApiLayerProperties(0, &LayerPropertyCount, nullptr)))
	{
		// As per EnumerateExtensions - a failure here means no runtime installed.
		return false;
	}

	if (!LayerPropertyCount)
	{
		// It's still legit if we have no layers, so early out here (and return success) if so.
		return true;
	}

	TArray<XrApiLayerProperties> LayerProperties;
	LayerProperties.SetNum(LayerPropertyCount);
	for (auto& Prop : LayerProperties)
	{
		Prop = XrApiLayerProperties{ XR_TYPE_API_LAYER_PROPERTIES };
	}

	if (XR_ENSURE(xrEnumerateApiLayerProperties(LayerPropertyCount, &LayerPropertyCount, LayerProperties.GetData())))
	{
		for (const auto& Prop : LayerProperties)
		{
			AvailableLayers.Add(Prop.layerName);
		}
		return true;
	}

	return false;
}

namespace OpenXRHMDModule
{
	struct AnsiKeyFunc : BaseKeyFuncs<const ANSICHAR*, const ANSICHAR*, false>
	{
		typedef typename TTypeTraits<const ANSICHAR*>::ConstPointerType KeyInitType;
		typedef typename TCallTraits<const ANSICHAR*>::ParamType ElementInitType;

		/**
		 * @return The key used to index the given element.
		 */
		static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
		{
			return Element;
		}

		/**
		 * @return True if the keys match.
		 */
		static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
		{
			return FCStringAnsi::Strcmp(A, B) == 0;
		}

		/** Calculates a hash index for a key. */
		static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
		{
			return UE::HashStringFNV1a32(MakeStringView(Key));
		}
	};
}


bool FOpenXRHMDModule::InitRenderBridge()
{
	// Get all extension plugins
	TSet<const ANSICHAR*, OpenXRHMDModule::AnsiKeyFunc> ExtensionSet;
	TArray<IOpenXRExtensionPlugin*> ExtModules = IModularFeatures::Get().GetModularFeatureImplementations<IOpenXRExtensionPlugin>(IOpenXRExtensionPlugin::GetModularFeatureName());

	// Query all extension plugins to see if we need to use a custom render bridge
	PFN_xrGetInstanceProcAddr GetProcAddr = nullptr;
	for (IOpenXRExtensionPlugin* Plugin : ExtModules)
	{
		// We are taking ownership of the CustomRenderBridge instance here.
		TRefCountPtr<FOpenXRRenderBridge> CustomRenderBridge = Plugin->GetCustomRenderBridge(Instance);
		if (CustomRenderBridge)
		{
			// We pick the first
			RenderBridge = CustomRenderBridge;
			return true;
		}
	}

	if (GDynamicRHI == nullptr)
	{
		return false;
	}

	if (!InitInstance())
	{
		return false;
	}

	const ERHIInterfaceType RHIType = RHIGetInterfaceType();

#ifdef XR_USE_GRAPHICS_API_D3D11
	if (RHIType == ERHIInterfaceType::D3D11 && IsExtensionEnabled(XR_KHR_D3D11_ENABLE_EXTENSION_NAME))
	{
		RenderBridge = CreateRenderBridge_D3D11(Instance);
		return true;
	}
	else
#endif
#ifdef XR_USE_GRAPHICS_API_D3D12
	if (RHIType == ERHIInterfaceType::D3D12 && IsExtensionEnabled(XR_KHR_D3D12_ENABLE_EXTENSION_NAME))
	{
		RenderBridge = CreateRenderBridge_D3D12(Instance);
		return true;
	}
	else
#endif
#if defined(XR_USE_GRAPHICS_API_OPENGL_ES) && defined(XR_USE_PLATFORM_ANDROID)
	if (RHIType == ERHIInterfaceType::OpenGL && IsExtensionEnabled(XR_KHR_OPENGL_ES_ENABLE_EXTENSION_NAME))
	{
		RenderBridge = CreateRenderBridge_OpenGLES(Instance);
		return true;
	}
	else
#endif
#ifdef XR_USE_GRAPHICS_API_OPENGL
	if (RHIType == ERHIInterfaceType::OpenGL && IsExtensionEnabled(XR_KHR_OPENGL_ENABLE_EXTENSION_NAME))
	{
		RenderBridge = CreateRenderBridge_OpenGL(Instance);
		return true;
	}
	else
#endif
#ifdef XR_USE_GRAPHICS_API_VULKAN
	if (RHIType == ERHIInterfaceType::Vulkan && IsExtensionEnabled(XR_KHR_VULKAN_ENABLE_EXTENSION_NAME))
	{
		RenderBridge = CreateRenderBridge_Vulkan(Instance);
		return true;
	}
	else
#endif
	{
		FString RHIString = FApp::GetGraphicsRHI();
		UE_LOGF(LogHMD, Warning, "%ls is not currently supported by the OpenXR runtime", *RHIString);
		return false;
	}
}

PFN_xrGetInstanceProcAddr FOpenXRHMDModule::GetDefaultLoader()
{
#if PLATFORM_WINDOWS
#if PLATFORM_CPU_ARM_FAMILY
	FString BinariesPath = FPaths::EngineDir() / FString(TEXT("Binaries/ThirdParty/OpenXR/WinArm64"));
#else
	FString BinariesPath = FPaths::EngineDir() / FString(TEXT("Binaries/ThirdParty/OpenXR/win64"));
#endif

	FString LoaderName = "openxr_loader.dll";
	FPlatformProcess::PushDllDirectory(*BinariesPath);
	LoaderHandle = FPlatformProcess::GetDllHandle(*LoaderName);
	FPlatformProcess::PopDllDirectory(*BinariesPath);
#elif PLATFORM_LINUX

#if PLATFORM_LINUXARM64
	FString BinariesPath = FPaths::EngineDir() / FString(TEXT("Binaries/ThirdParty/OpenXR/LinuxArm64/aarch64-unknown-linux-gnueabi"));
	FString LoaderName = "libopenxr_loader.so";
	LoaderHandle = FPlatformProcess::GetDllHandle(*(BinariesPath / LoaderName)); 
#else
	FString BinariesPath = FPaths::EngineDir() / FString(TEXT("Binaries/ThirdParty/OpenXR/linux/x86_64-unknown-linux-gnu"));
	FString LoaderName = "libopenxr_loader.so";
	LoaderHandle = FPlatformProcess::GetDllHandle(*(BinariesPath / LoaderName)); 
#endif

#elif PLATFORM_ANDROID
	FString LoaderName = "libopenxr_loader.so";
	LoaderHandle = FPlatformProcess::GetDllHandle(*LoaderName);
#endif

	if (!LoaderHandle)
	{
		UE_LOGF(LogHMD, Log, "Failed to find OpenXR runtime loader.");
		return nullptr;
	}

	PFN_xrGetInstanceProcAddr OutGetProcAddr = (PFN_xrGetInstanceProcAddr)FPlatformProcess::GetDllExport(LoaderHandle, TEXT("xrGetInstanceProcAddr"));

#if PLATFORM_ANDROID
	PFN_xrInitializeLoaderKHR xrInitializeLoaderKHR;
	OutGetProcAddr(XR_NULL_HANDLE, "xrInitializeLoaderKHR", (PFN_xrVoidFunction*)&xrInitializeLoaderKHR);
	if (xrInitializeLoaderKHR == nullptr)
	{
		UE_LOG(LogHMD, Error, TEXT("Unable to load OpenXR Android xrInitializeLoaderKHR"));
		return nullptr;
	}

	XrLoaderInitInfoAndroidKHR LoaderInitializeInfoAndroid;
	LoaderInitializeInfoAndroid.type = XR_TYPE_LOADER_INIT_INFO_ANDROID_KHR;
	LoaderInitializeInfoAndroid.next = NULL;
	LoaderInitializeInfoAndroid.applicationVM = GNativeAndroidApp->activity->vm;
	LoaderInitializeInfoAndroid.applicationContext = GNativeAndroidApp->activity->clazz;
	XR_ENSURE(xrInitializeLoaderKHR((XrLoaderInitInfoBaseHeaderKHR*)&LoaderInitializeInfoAndroid));
#endif
	return OutGetProcAddr;
}

bool FOpenXRHMDModule::GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
#if PLATFORM_ANDROID
	OutExtensions.Add(XR_KHR_ANDROID_CREATE_INSTANCE_EXTENSION_NAME);
#endif

	// If the commandline -xrtrackingonly is passed, then start the application in _Other mode instead of _Scene mode
	// This is used when we only want to get tracking information and don't need to render anything to the XR device
	if (FParse::Param(FCommandLine::Get(), TEXT("xrtrackingonly")))
	{
		OutExtensions.Add(XR_MND_HEADLESS_EXTENSION_NAME);
	}

	return true;
}

bool FOpenXRHMDModule::GetOptionalExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
#ifdef XR_USE_GRAPHICS_API_D3D11
	OutExtensions.Add(XR_KHR_D3D11_ENABLE_EXTENSION_NAME);
#endif
#ifdef XR_USE_GRAPHICS_API_D3D12
	OutExtensions.Add(XR_KHR_D3D12_ENABLE_EXTENSION_NAME);
#endif
#ifdef XR_USE_GRAPHICS_API_OPENGL
	OutExtensions.Add(XR_KHR_OPENGL_ENABLE_EXTENSION_NAME);
#endif
#ifdef XR_USE_GRAPHICS_API_OPENGL_ES
	OutExtensions.Add(XR_KHR_OPENGL_ES_ENABLE_EXTENSION_NAME);
#endif
#ifdef XR_USE_GRAPHICS_API_VULKAN
	OutExtensions.Add(XR_KHR_VULKAN_ENABLE_EXTENSION_NAME);
	OutExtensions.Add(XR_KHR_VULKAN_SWAPCHAIN_FORMAT_LIST_EXTENSION_NAME);
	OutExtensions.Add(XR_FB_FOVEATION_VULKAN_EXTENSION_NAME);
	OutExtensions.Add(XR_META_VULKAN_SWAPCHAIN_CREATE_INFO_EXTENSION_NAME);
#endif
	OutExtensions.Add(XR_KHR_COMPOSITION_LAYER_DEPTH_EXTENSION_NAME);
	OutExtensions.Add(XR_FB_COMPOSITION_LAYER_DEPTH_TEST_EXTENSION_NAME);
	OutExtensions.Add(XR_KHR_COMPOSITION_LAYER_CYLINDER_EXTENSION_NAME);
	OutExtensions.Add(XR_KHR_COMPOSITION_LAYER_EQUIRECT_EXTENSION_NAME);
	OutExtensions.Add(XR_KHR_COMPOSITION_LAYER_EQUIRECT2_EXTENSION_NAME);
	OutExtensions.Add(XR_KHR_COMPOSITION_LAYER_CUBE_EXTENSION_NAME);
	OutExtensions.Add(XR_KHR_COMPOSITION_LAYER_COLOR_SCALE_BIAS_EXTENSION_NAME);
	OutExtensions.Add(XR_VARJO_QUAD_VIEWS_EXTENSION_NAME);
	OutExtensions.Add(XR_KHR_VISIBILITY_MASK_EXTENSION_NAME);
	OutExtensions.Add(XR_KHR_BINDING_MODIFICATION_EXTENSION_NAME);
	OutExtensions.Add(XR_EPIC_VIEW_CONFIGURATION_FOV_EXTENSION_NAME);
	OutExtensions.Add(XR_EXT_DPAD_BINDING_EXTENSION_NAME);
	OutExtensions.Add(XR_EXT_PALM_POSE_EXTENSION_NAME);
	OutExtensions.Add(XR_EXT_ACTIVE_ACTION_SET_PRIORITY_EXTENSION_NAME);
	OutExtensions.Add(XR_FB_FOVEATION_EXTENSION_NAME);
	OutExtensions.Add(XR_FB_SWAPCHAIN_UPDATE_STATE_EXTENSION_NAME);
	OutExtensions.Add(XR_FB_FOVEATION_CONFIGURATION_EXTENSION_NAME); 
#if PLATFORM_WINDOWS
	OutExtensions.Add(XR_OCULUS_AUDIO_DEVICE_GUID_EXTENSION_NAME);
#endif

	// These extensions are mutually incompatible because XR_FB_composition_layer_alpha_blend disables non-opaque environment blend modes
	// XR_EXT_composition_layer_inverted_alpha is the more modern and platform-agnostic option and is preferred when available
	if (AvailableExtensions.Contains(XR_EXT_COMPOSITION_LAYER_INVERTED_ALPHA_EXTENSION_NAME))
	{
		OutExtensions.Add(XR_EXT_COMPOSITION_LAYER_INVERTED_ALPHA_EXTENSION_NAME);
	}
	else if (AvailableExtensions.Contains(XR_FB_COMPOSITION_LAYER_ALPHA_BLEND_EXTENSION_NAME))
	{
		OutExtensions.Add(XR_FB_COMPOSITION_LAYER_ALPHA_BLEND_EXTENSION_NAME);
	}

	// These extensions both perform frame synthesis, and require providing motion vector and motion vector depth swapchains at a requested size
	// XR_EXT_frame_synthesis is the more modern and platform-agnostic option and is preferred when available
	// However, it is not currently working properly on Oculus runtimes, so XR_FB_space_warp is preferred on Quest devices
	const bool bSpaceWarpSupported = AvailableExtensions.Contains(XR_FB_SPACE_WARP_EXTENSION_NAME);
	const bool bFrameSynthesisSupported = AvailableExtensions.Contains(XR_EXT_FRAME_SYNTHESIS_EXTENSION_NAME);

	if (bSpaceWarpSupported && bFrameSynthesisSupported)
	{
		if (CVarPreferFBSpaceWarp.GetValueOnAnyThread())
		{
			OutExtensions.Add(XR_FB_SPACE_WARP_EXTENSION_NAME);
		}
		else
		{
			OutExtensions.Add(XR_EXT_FRAME_SYNTHESIS_EXTENSION_NAME);
		}
	}
	else
	{
		if (bFrameSynthesisSupported)
		{
			OutExtensions.Add(XR_EXT_FRAME_SYNTHESIS_EXTENSION_NAME);
		}
		else if (bSpaceWarpSupported)
		{
			OutExtensions.Add(XR_FB_SPACE_WARP_EXTENSION_NAME);
		}
	}
	
	OutExtensions.Add(XR_EXT_LOCAL_FLOOR_EXTENSION_NAME);
	OutExtensions.Add(XR_EXT_USER_PRESENCE_EXTENSION_NAME);
	return true;
}

namespace OpenXRHMDModule
{
	typedef TSet<const ANSICHAR*, OpenXRHMDModule::AnsiKeyFunc> ExtensionSet;

	bool EnableExtensions(const TSet<FString>& AvailableExtensions, const TArray<const ANSICHAR*>& RequiredExtensions, const TArray<const ANSICHAR*>& OptionalExtensions, TArray<const ANSICHAR*>& OutExtensions, const TSet<FString>& FullyBlockedExtensionsSet, const ExtensionSet& BlockedOptionalExtensionSet)
	{
		// Query required extensions and check if they're all available
		bool ExtensionMissing = false;
		for (const ANSICHAR* Ext : RequiredExtensions)
		{
			if (AvailableExtensions.Contains(Ext))
			{
				if (FullyBlockedExtensionsSet.Contains(Ext))
				{
					UE_LOGF(LogHMD, Warning, "Required extension %s is blocked by FullyBlockedExtensionsSet", Ext);
					ExtensionMissing = true;
				}
				else
				{
					UE_LOGF(LogHMD, Verbose, "Required extension %s enabled", Ext);
				}
			}
			else
			{
				UE_LOGF(LogHMD, Warning, "Required extension %s is not available", Ext);
				ExtensionMissing = true;
			}
		}

		// If any required extensions are missing then we ignore the plugin
		if (ExtensionMissing)
		{
			return false;
		}

		// All required extensions are supported we can safely add them to our set and give the plugin callbacks
		OutExtensions.Append(RequiredExtensions);

		// Add all supported optional extensions to the set, unless blocked.
		for (const ANSICHAR* Ext : OptionalExtensions)
		{
			if (AvailableExtensions.Contains(Ext))
			{
				if (FullyBlockedExtensionsSet.Contains(Ext))
				{
					UE_LOGF(LogHMD, Log, "Optional extension %s is blocked by FullyBlockedExtensionsSet", Ext);
				}
				else if (BlockedOptionalExtensionSet.Contains(Ext))
				{
					UE_LOGF(LogHMD, Log, "Optional extension %s is blocked by BlockedOptionalExtensionSet", Ext);
				}
				else
				{
					UE_LOGF(LogHMD, Verbose, "Optional extension %s enabled", Ext);
					OutExtensions.Add(Ext);
				}
			}
			else
			{
				UE_LOGF(LogHMD, Log, "Optional extension %s is not available", Ext);
			}
		}

		return true;
	}
	
	void LogExtensions(bool DoLog, const TCHAR* Preface, const TArray<const ANSICHAR*>& Extensions)
	{
		if (DoLog)
		{
			UE_LOGF(LogHMD, Log, "%ls", Preface);
			for (const ANSICHAR* Extension : Extensions)
			{
				UE_LOGF(LogHMD, Log, "      %s", Extension);
			}
		}
	}
	void LogExtensionsVerbose(bool DoLog, const TCHAR* Preface, const TArray<const ANSICHAR*>& Extensions)
	{
		if (DoLog)
		{
			UE_LOGF(LogHMD, Verbose, "%ls", Preface);
			for (const ANSICHAR* Extension : Extensions)
			{
				UE_LOGF(LogHMD, Verbose, "      %s", Extension);
			}
		}
	}
}

bool FOpenXRHMDModule::InitInstance()
{
	if (Instance)
	{
		return true;
	}

#if PLATFORM_ANDROID
	if (AndroidThunkCpp_IsOculusMobileApplication())
	{
		UE_LOG(LogHMD, Log, TEXT("OpenXRHMDModule: App is packaged for Oculus/Meta Mobile OpenXR"));
	}
	else if (AndroidThunkCpp_IsXRImmersiveHMDApplication())
	{
		UE_LOGF(LogHMD, Log, "OpenXRHMDModule: App is packaged for Android OpenXR ImmersiveHMD");
	}
	else
	{
		UE_LOGF(LogHMD, Log, "OpenXRHMDModule: OpenXR disabled because this app is not packaged for OpenXR (see android settings: bPackageForMetaQuest, bPackageForOpenXRImmersive, etc");
		return false;
	}
#endif

	// Get all extension plugins
	OpenXRHMDModule::ExtensionSet ExtensionSet;
	TArray<IOpenXRExtensionPlugin*> ExtModules = IModularFeatures::Get().GetModularFeatureImplementations<IOpenXRExtensionPlugin>(IOpenXRExtensionPlugin::GetModularFeatureName());

	// Query all extension plugins to see if we need to use a custom loader
	PFN_xrGetInstanceProcAddr GetProcAddr = nullptr;
	for (IOpenXRExtensionPlugin* Plugin : ExtModules)
	{
		if (Plugin->GetCustomLoader(&GetProcAddr))
		{
			// We pick the first loader we can find
			UE_LOGF(LogHMD, Log, "OpenXRHMDModule::InitInstance found and will use CustomLoader from plugin %ls", *Plugin->GetDisplayName());
			break;
		}

		// Clear it again just to ensure the failed call didn't leave the pointer set
		GetProcAddr = nullptr;
	}

	if (!GetProcAddr)
	{
		UE_LOGF(LogHMD, Log, "OpenXRHMDModule::InitInstance using DefaultLoader.");
		GetProcAddr = GetDefaultLoader();
	}

	for (IOpenXRExtensionPlugin* Plugin : ExtModules)
	{
		if (Plugin->InsertOpenXRAPILayer(GetProcAddr))
		{
			UE_LOGF(LogHMD, Log, "IOpenXRExtensionPlugin API layer enabled: %ls", *Plugin->GetDisplayName());
		}
	}

	if (!PreInitOpenXRCore(GetProcAddr))
	{
		UE_LOGF(LogHMD, Log, "Failed to initialize core functions. Please check that you have a valid OpenXR runtime installed.");
		return false;
	}

	if (!EnumerateExtensions())
	{
		UE_LOGF(LogHMD, Log, "Failed to enumerate extensions. Please check that you have a valid OpenXR runtime installed.");
		return false;
	}

	if (!EnumerateLayers())
	{
		UE_LOGF(LogHMD, Log, "Failed to enumerate API layers. Please check that you have a valid OpenXR runtime installed.");
		return false;
	}

	// Gather set of extensions that are blocked by OpenXRHMDSettings BlockedExtensions setting/ini.
	TSet<FString> FullyBlockedExtensionsSet;
	{
		TArray<FString> BlockedExtensions;
		if (GConfig->GetArray(TEXT("/Script/OpenXRHMD.OpenXRHMDSettings"), TEXT("BlockedExtensions"), BlockedExtensions, GEngineIni))
		{
			FullyBlockedExtensionsSet.Append(BlockedExtensions);
		}
	}
	if (!FullyBlockedExtensionsSet.IsEmpty())
	{
		UE_LOGF(LogHMD, Log, "FullyBlockedExtensionsSet:");
		for (const FString& BlockedExtension : FullyBlockedExtensionsSet)
		{
			UE_LOGF(LogHMD, Log, "  %ls", *BlockedExtension);
		}
	}

	// Gather set of optional extensions that are blocked by extension plugins
	OpenXRHMDModule::ExtensionSet BlockedOptionalExtensionSet;
	{
		TMap<IOpenXRExtensionPlugin*, TArray<const ANSICHAR*>> BlockedOptionalExtensionMap;
		for (IOpenXRExtensionPlugin* Plugin : ExtModules)
		{
			TArray<const ANSICHAR*> BlockedOptionalExtensions;
			bool RetVal = Plugin->GetBlockedOptionalExtensions(BlockedOptionalExtensions);
			if (RetVal && BlockedOptionalExtensions.Num() > 0)
			{
				BlockedOptionalExtensionMap.Add(Plugin, BlockedOptionalExtensions);
				for (const ANSICHAR* Blocked : BlockedOptionalExtensions)
				{
					BlockedOptionalExtensionSet.Add(Blocked);
				}
			}
		}
		// Logging to show which extension plugins are blocking what optional extensions.
		if (!BlockedOptionalExtensionMap.IsEmpty())
		{
			UE_LOGF(LogHMD, Log, "BlockedOptionalExtensionSet:");
			for (TPair< IOpenXRExtensionPlugin*, TArray<const ANSICHAR*> > Pair : BlockedOptionalExtensionMap)
			{
				UE_LOGF(LogHMD, Log, "  %ls is blocking:", *(Pair.Key->GetDisplayName()));
				for (const ANSICHAR* Extension : Pair.Value)
				{
					UE_LOGF(LogHMD, Log, "    %s", Extension);
				}
			}
		}
	}

	// Enable any required and optional extensions that are not plugin specific (usually platform support extensions)
	{
		UE_LOGF(LogHMD, Log, "Building UE core extension lists:");

		TArray<const ANSICHAR*> RequiredExtensions, OptionalExtensions, Extensions;
		// Query required extensions
		RequiredExtensions.Empty();
		if (!GetRequiredExtensions(RequiredExtensions))
		{
			UE_LOGF(LogHMD, Error, "Could not get required OpenXR extensions.");
			return false;
		}
		OpenXRHMDModule::LogExtensionsVerbose(true, TEXT("  RequiredExtensions"), RequiredExtensions);

		// Query optional extensions
		OptionalExtensions.Empty();
		if (!GetOptionalExtensions(OptionalExtensions))
		{
			UE_LOGF(LogHMD, Error, "Could not get optional OpenXR extensions.");
			return false;
		}
		OpenXRHMDModule::LogExtensionsVerbose(true, TEXT("  OptionalExtensions"), OptionalExtensions);

		if (!OpenXRHMDModule::EnableExtensions(AvailableExtensions, RequiredExtensions, OptionalExtensions, Extensions, FullyBlockedExtensionsSet, BlockedOptionalExtensionSet))
		{
			UE_LOGF(LogHMD, Error, "Could not enable all required OpenXR extensions.");
			return false;
		}
		ExtensionSet.Append(Extensions);
	}

	UE_LOGF(LogHMD, Log, "Extensions for IOpenXRExtensionPlugins:");
	ExtensionPlugins.Reset();
	for (IOpenXRExtensionPlugin* Plugin : ExtModules)
	{
		UE_LOGF(LogHMD, Log, "  %ls:", *Plugin->GetDisplayName());
		TArray<const ANSICHAR*> RequiredExtensions, OptionalExtensions, Extensions;

		// Query required extensions
		RequiredExtensions.Empty();
		if (!Plugin->GetRequiredExtensions(RequiredExtensions))
		{
			// Ignore the plugin if the query fails
			continue;
		}
		OpenXRHMDModule::LogExtensionsVerbose(!RequiredExtensions.IsEmpty(), TEXT("    RequiredExtensions"), RequiredExtensions);

		// Query optional extensions
		OptionalExtensions.Empty();
		if (!Plugin->GetOptionalExtensions(OptionalExtensions))
		{
			// Ignore the plugin if the query fails
			continue;
		}
		OpenXRHMDModule::LogExtensionsVerbose(!OptionalExtensions.IsEmpty(), TEXT("    OptionalExtensions"), OptionalExtensions);

		if (!OpenXRHMDModule::EnableExtensions(AvailableExtensions, RequiredExtensions, OptionalExtensions, Extensions, FullyBlockedExtensionsSet, BlockedOptionalExtensionSet))
		{
			// Ignore the plugin if the required extension could not be enabled
			FString ModuleName = Plugin->GetDisplayName();
			UE_LOGF(LogHMD, Log, "Could not enable all required OpenXR extensions for %ls on current system. This plugin will be loaded but ignored, but will be enabled on a target platform that supports the required extension.", *ModuleName);
			continue;
		}
		ExtensionSet.Append(Extensions);
		ExtensionPlugins.Add(Plugin);
	}

	if (auto ARModule = FModuleManager::LoadModulePtr<IOpenXRARModule>("OpenXRAR"))
	{
		TArray<const ANSICHAR*> ARExtensionSet;
		ARModule->GetExtensions(ARExtensionSet);
		ExtensionSet.Append(ARExtensionSet);
	}

	EnabledExtensions.Reset();
	for (const ANSICHAR* Ext : ExtensionSet)
	{
		EnabledExtensions.Add(Ext);
	}
	
	OpenXRHMDModule::LogExtensions(true, TEXT("    EnabledExtensions:"), EnabledExtensions);

	// Enable layers, if specified by CVar.
	// Note: For the validation layer to work on Windows (as of latest OpenXR runtime, August 2019), the following are required:
	//   1. Download and build the OpenXR SDK from https://github.com/KhronosGroup/OpenXR-SDK-Source (follow instructions at https://github.com/KhronosGroup/OpenXR-SDK-Source/blob/main/BUILDING.md)
	//	 2. Add a registry key under HKEY_LOCAL_MACHINE\SOFTWARE\Khronos\OpenXR\1\ApiLayers\Explicit, containing the path to the manifest file
	//      (e.g. C:\OpenXR-SDK-Source-main\build\win64\src\api_layers\XrApiLayer_core_validation.json) <-- this file is downloaded as part of the SDK source, above
	//   3. Copy the DLL from the build target at, for example, C:\OpenXR-SDK-Source-main\build\win64\src\api_layers\XrApiLayer_core_validation.dll to
	//      somewhere in your system path (e.g. c:\windows\system32); the OpenXR loader currently doesn't use the path the json file is in (this is a bug)

	const bool bEnableOpenXRValidationLayer = (CVarEnableOpenXRValidationLayer.GetValueOnAnyThread() != 0)
		|| FParse::Param(FCommandLine::Get(), TEXT("openxrdebug"))
		|| FParse::Param(FCommandLine::Get(), TEXT("openxrvalidation"));
	TArray<const char*> Layers;
	if (bEnableOpenXRValidationLayer)
	{
		if (AvailableLayers.Contains("XR_APILAYER_LUNARG_core_validation"))
		{
			UE_LOGF(LogHMD, Display, "Running with OpenXR validation layers, performance might be degraded.");
			Layers.Add("XR_APILAYER_LUNARG_core_validation");
		}
		else
		{
			UE_LOGF(LogHMD, Error, "OpenXR validation was requested, but the validation layer isn't available. Request ignored.");
		}
	}

	// Engine registration can be disabled via console var.
	auto* CVarDisableEngineAndAppRegistration = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DisableEngineAndAppRegistration"));
	bool bDisableEngineRegistration = (CVarDisableEngineAndAppRegistration && CVarDisableEngineAndAppRegistration->GetValueOnAnyThread() != 0);

	// EngineName will be of the form "UnrealEngine4.21", with the minor version ("21" in this example)
	// updated with every quarterly release
	FString EngineName = bDisableEngineRegistration ? FString("") : FApp::GetEpicProductIdentifier() + FEngineVersion::Current().ToString(EVersionComponent::Minor);
	FString AppName = bDisableEngineRegistration ? FString("") : FApp::GetProjectName();

	XrInstanceCreateInfo Info;
	Info.type = XR_TYPE_INSTANCE_CREATE_INFO;
	Info.next = nullptr;
	Info.createFlags = 0;
	FPlatformString::Convert((UTF8CHAR*)Info.applicationInfo.applicationName, XR_MAX_APPLICATION_NAME_SIZE, *AppName, AppName.Len() + 1);
	Info.applicationInfo.applicationVersion = static_cast<uint32>(BuildSettings::GetCurrentChangelist()) | (BuildSettings::IsLicenseeVersion() ? 0x80000000 : 0);
	FPlatformString::Convert((UTF8CHAR*)Info.applicationInfo.engineName, XR_MAX_ENGINE_NAME_SIZE, *EngineName, EngineName.Len() + 1);
	Info.applicationInfo.engineVersion = (uint32)(FEngineVersion::Current().GetMinor() << 16 | FEngineVersion::Current().GetPatch());


	Info.applicationInfo.apiVersion = 0; // We will overwrite this setting in TryCreateInstance and potentially try multiple versions
	Info.enabledApiLayerCount = Layers.Num();
	Info.enabledApiLayerNames = Layers.GetData();

	Info.enabledExtensionCount = EnabledExtensions.Num();
	Info.enabledExtensionNames = EnabledExtensions.GetData();

#if PLATFORM_ANDROID
	XrInstanceCreateInfoAndroidKHR InstanceCreateInfoAndroid;
	InstanceCreateInfoAndroid.type = XR_TYPE_INSTANCE_CREATE_INFO_ANDROID_KHR;
	InstanceCreateInfoAndroid.next = nullptr;
	InstanceCreateInfoAndroid.applicationVM = GNativeAndroidApp->activity->vm;
	InstanceCreateInfoAndroid.applicationActivity = GNativeAndroidApp->activity->clazz;
	Info.next = &InstanceCreateInfoAndroid;
#endif // PLATFORM_ANDROID

	if (!TryCreateInstance(Info))
	{
		return false;
	}

	if (!InitOpenXRCore(Instance))
	{
		UE_LOGF(LogHMD, Log, "Failed to initialize core functions. Please check that you have a valid OpenXR runtime installed.");
		return false;
	}

	XrInstanceProperties InstanceProps = { XR_TYPE_INSTANCE_PROPERTIES, nullptr };
	XR_ENSURE(xrGetInstanceProperties(Instance, &InstanceProps));
	InstanceProps.runtimeName[XR_MAX_RUNTIME_NAME_SIZE - 1] = 0; // Ensure the name is null terminated.
	UE_LOGF(LogHMD, Log, "Initialized OpenXR on %s runtime version %d.%d.%d", InstanceProps.runtimeName, XR_VERSION_MAJOR(InstanceProps.runtimeVersion), XR_VERSION_MINOR(InstanceProps.runtimeVersion), XR_VERSION_PATCH(InstanceProps.runtimeVersion));

	FGenericCrashContext::SetEngineData(TEXT("XR"), FString::Printf(TEXT("XR OpenXRHMD %hs %d.%d.%d"), InstanceProps.runtimeName, XR_VERSION_MAJOR(InstanceProps.runtimeVersion), XR_VERSION_MINOR(InstanceProps.runtimeVersion), XR_VERSION_PATCH(InstanceProps.runtimeVersion)));

	if (CVarCheckOpenXRInstanceConformance.GetValueOnAnyThread() &&
		(FCStringAnsi::Strstr(InstanceProps.runtimeName, "SteamVR/OpenXR") != nullptr))
	{
		// Runtimes should not be dependent on system availability to use instance-only functions.
		// However, some runtimes fail with some instance-only calls, such as xrStringToPath. We
		// need to bail early to prevent failures later on during setup.
		XrPath UserHeadTestPath = XR_NULL_PATH;
		const XrResult StringToPathTest = xrStringToPath(Instance, "/user/head", &UserHeadTestPath);

		if (StringToPathTest != XR_SUCCESS)
		{
			UE_LOGF(LogHMD, Warning, "Instance does not support expected usage of xrStringToPath. Instance is not viable.");
			return false;
		}
	}

	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Module->PostCreateInstance(Instance);
	}

	return true;
}

struct FProblematicOpenXRApiLayerInfo
{
	FProblematicOpenXRApiLayerInfo(const FString& String)
	{
		FString StringCopy = String;
		StringCopy.RemoveSpacesInline();
		FParse::Value(*String, TEXT("Name="), Name);
		FParse::Value(*String, TEXT("MinVersion="), MinVersion);
		FParse::Value(*String, TEXT("MaxVersion="), MaxVersion);
		FParse::Value(*String, TEXT("ExtensionAddedToFallbackWithout="), ExtensionAddedToFallbackWithout);
	}
	FString Name;
	int MinVersion = 0;
	int MaxVersion = 0; // 0 for both = all.
	FString ExtensionAddedToFallbackWithout;  // If this extension is available and instance creation fails with XR_ERROR_EXTENSION_NOT_PRESENT we will try again without it.
};

bool FOpenXRHMDModule::TryCreateInstance(XrInstanceCreateInfo& Info)
{
	// Cache a copy of the info not modified by ExtensionPlugins, some of which we might have to disable.
	XrInstanceCreateInfo LocalInfo = Info;

	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Info.next = Module->OnCreateInstance(this, Info.next);
	}

	// Try each api version that is enabled last to first.
	bool bIsOpenXR1_1Enabled = true;
	bool bIsOpenXR1_0Enabled = true;
	{
		// Read the ini.  This code can run so early that we cannot use a default object to load this more easily.
		// Note the settings are true by default, so if we don't find a setting we use true.
		bool bFoundValue = false;
		if (GConfig->GetBool(TEXT("/Script/OpenXRHMD.OpenXRHMDSettings"), TEXT("bIsOpenXR1_1Enabled"), bFoundValue, GEngineIni))
		{
			bIsOpenXR1_1Enabled = bFoundValue;
		}
		if (GConfig->GetBool(TEXT("/Script/OpenXRHMD.OpenXRHMDSettings"), TEXT("bIsOpenXR1_0Enabled"), bFoundValue, GEngineIni))
		{
			bIsOpenXR1_0Enabled = bFoundValue;
		}
	}

	XrResult Result = XR_ERROR_API_VERSION_UNSUPPORTED;
	if (bIsOpenXR1_1Enabled)
	{
		Info.applicationInfo.apiVersion = XR_API_VERSION_1_1;
		Result = xrCreateInstance(&Info, &Instance);
		if (XR_SUCCEEDED(Result))
		{
			UE_LOGF(LogHMD, Log, "xrCreateInstance succeeded with XR_API_VERSION_1_1");
			InstanceOpenXRAPIVersion = EOpenXRAPIVersion::V_1_1;
		}
	}

	if (Result == XR_ERROR_API_VERSION_UNSUPPORTED && bIsOpenXR1_0Enabled)
	{
		Info.applicationInfo.apiVersion = XR_API_VERSION_1_0;
		Result = xrCreateInstance(&Info, &Instance);
		if (XR_SUCCEEDED(Result))
		{
			UE_LOGF(LogHMD, Log, "xrCreateInstance succeeded with XR_API_VERSION_1_0");
			InstanceOpenXRAPIVersion = EOpenXRAPIVersion::V_1_0;
		}
	}

	// Attempt to deal with problematic extensions
	if (Result == XR_ERROR_EXTENSION_NOT_PRESENT)
	{
		// An extension we requested is not supported, but we normally only add extensions that must be supported and extensions that were listed to us by the runtime, so how did we get into this situation?
		// A badly behaving layer might add an extension into the available extensions list, but then not remove it before the runtime sees that list causing the runtime to fail instance creation with XR_ERROR_EXTENSION_NOT_PRESENT.
		// Therefore we have a ProblematicOpenXRApiLayerInfos config setting to define layers that might give us bad extensions which we can then try to create an instance without.

		// Read the ini.  This code can run so early that we cannot use a default object to load this more easily.
		TArray<FProblematicOpenXRApiLayerInfo> ProblematicOpenXRApiLayerInfos;
		{
			TArray<FString> ProblematicOpenXRApiLayerInfoStrings;
			GConfig->GetArray(TEXT("OpenXR"), TEXT("ProblematicOpenXRApiLayerInfos"), ProblematicOpenXRApiLayerInfoStrings, GEngineIni);
			ProblematicOpenXRApiLayerInfos.Reserve(ProblematicOpenXRApiLayerInfoStrings.Num());
			for (FString& Str : ProblematicOpenXRApiLayerInfoStrings)
			{
				ProblematicOpenXRApiLayerInfos.Emplace(Str);
			}
		}

		if (ProblematicOpenXRApiLayerInfos.Num() > 0)
		{
			// Copy the currently active ProblematicLayerInfos that have ExtensionsAddedToFallbackWithout into a new array.
			TArray<const FProblematicOpenXRApiLayerInfo*> ActiveProblematicLayerInfos;
			{
				TArray<XrApiLayerProperties> ActiveLayerProperties;
				EnumerateOpenXRApiLayers(ActiveLayerProperties);
				TMap<FString, int> ActiveLayerPropertyNameMap;
				for (int i = 0; i < ActiveLayerProperties.Num(); ++i)
				{
					ActiveLayerPropertyNameMap.Add(ActiveLayerProperties[i].layerName, ActiveLayerProperties[i].layerVersion);
				}
				for (const FProblematicOpenXRApiLayerInfo& ProblematicLayerInfo : ProblematicOpenXRApiLayerInfos)
				{
					// Skip if no extension specified.
					if (ProblematicLayerInfo.ExtensionAddedToFallbackWithout.Len() == 0)
					{
						continue;
					}
					// Check if this one is active.
					int* Found = ActiveLayerPropertyNameMap.Find(ProblematicLayerInfo.Name);
					if (Found)
					{
						if ((ProblematicLayerInfo.MinVersion == 0 && ProblematicLayerInfo.MaxVersion == 0) || (*Found >= ProblematicLayerInfo.MinVersion && *Found <= ProblematicLayerInfo.MaxVersion))
						{
							ActiveProblematicLayerInfos.Add(&ProblematicLayerInfo);
						}
					}
				}
			}

			if (ActiveProblematicLayerInfos.Num() > 0)
			{
				UE_LOGF(LogHMD, Log, "Failed to create an OpenXR instance with all of the extensions that are enabled.  The OpenXR runtime says it does not support all of those. "
					"We have a list of Problematic OpenXR ApiLayers that may be adding extensions that are not supported by the runtime. "  
					"We can try disabling those extensions to see if we can find a set that the runtime will create an instance for. "
					"Sadly this is a combinatorial process.");
				if (UE_LOG_ACTIVE(LogHMD, Log))
				{
					UE_LOGF(LogHMD, Log, "Problematic Layer Extension Information for currently active OpenXRApiLayers:");
					for (const FProblematicOpenXRApiLayerInfo* LayerInfo : ActiveProblematicLayerInfos)
					{
						UE_LOGF(LogHMD, Log, "  Layer: %ls  Version: %i-%i  Extension: %ls", *LayerInfo->Name, LayerInfo->MinVersion, LayerInfo->MaxVersion, *LayerInfo->ExtensionAddedToFallbackWithout);
					}
				}

				// Build a list of all of the problematic extensions, as ansi char pointers from the EnabledExtensions list.
				TArray<const char*> ProblematicExtensionList;
				{
					TSet<const FString*> ProblematicExtensionSet;
					for (const FProblematicOpenXRApiLayerInfo* LayerInfo : ActiveProblematicLayerInfos)
					{
						ProblematicExtensionSet.Add(&(LayerInfo->ExtensionAddedToFallbackWithout));
					}
					ProblematicExtensionList.Reserve(ProblematicExtensionSet.Num());
					for (const FString* ProblematicExtension : ProblematicExtensionSet)
					{
						auto ProblematicExtensionConverter = StringCast<ANSICHAR>(**ProblematicExtension);
						const char* ProblematicExtensionCStr = ProblematicExtensionConverter.Get();
						for (uint32_t i = 0; i < Info.enabledExtensionCount; ++i)
						{
							const char* EnabledExtension = Info.enabledExtensionNames[i];
							if (FCStringAnsi::Strncmp(ProblematicExtensionCStr, EnabledExtension, XR_MAX_EXTENSION_NAME_SIZE) == 0)
							{
								ProblematicExtensionList.Add(EnabledExtension);
							}
						}
					}
				}

				// Create a list of enabled extensions excluding the problematic ones.
				TArray<const char*> LocalEnabledExtensions;
				LocalEnabledExtensions.Reserve(Info.enabledExtensionCount);
				for (uint32_t i = 0; i < Info.enabledExtensionCount; i++)
				{
					const char* EnabledExtension = Info.enabledExtensionNames[i];
					if (!ProblematicExtensionList.Contains(EnabledExtension))
					{
						LocalEnabledExtensions.Add(EnabledExtension);
					}
				}

				// Try to xrCreateInstance with each combination until one succeeds or all fail							
				
				// So now we have a list of non-problematic extensions (LocalEnabledExtensions) and a list of problematic ones (ProblematicExtensionList).  
				// We want to try enabling the problematic extensions, exploring all combinations, and last we will try enabling none of them.
				// This whole algorithm is N^2, but N is actually always 1 right now.  We will just clamp N to 3 and assert.  If we actually want to use it with more
				// It'll need some kind of rewrite.
				const int StartIndex = LocalEnabledExtensions.Num();
				int32 NumProblematicExtensions = ProblematicExtensionList.Num();
				check(NumProblematicExtensions < 64);
				check(NumProblematicExtensions <= 3);
				NumProblematicExtensions = FMath::Min(NumProblematicExtensions, 3);
				uint64 Bitfield = (1ull << (NumProblematicExtensions)) - 1;  // Get a 1 for each extension.
				do
				{
					--Bitfield; // We do this first because we already tried with all enabled.

					// Enable some of the problematic extensions
					LocalEnabledExtensions.SetNum(StartIndex, EAllowShrinking::No); // Shrink off problematic ones from previous iterations
					for (int i = 0; i < NumProblematicExtensions; ++i)
					{
						if (((Bitfield >> i) & 1) == 1)
						{
							LocalEnabledExtensions.Add(ProblematicExtensionList[i]);
						}
					}

					// Fill in the Instance creation info struct
					LocalInfo.enabledExtensionCount = LocalEnabledExtensions.Num();
					LocalInfo.enabledExtensionNames = LocalEnabledExtensions.GetData();

					// Figure out which extensionplugins need to be disabled because their required extensions are disabled.
					TArray<IOpenXRExtensionPlugin*> LocalExtensionPlugins = ExtensionPlugins;
					TArray<IOpenXRExtensionPlugin*> LocallyDisabledExtensionPlugins;
					for (int i = LocalExtensionPlugins.Num() - 1; i >= 0 ; --i)
					{
						IOpenXRExtensionPlugin* ExtensionPlugin = LocalExtensionPlugins[i];

						TArray<const ANSICHAR*> RequiredExtensions;
						if (!ExtensionPlugin->GetRequiredExtensions(RequiredExtensions))
						{
							UE_LOGF(LogHMD, Error, "Could not get required OpenXR extensions.");
							return false;
						}
						bool bAllRequiredAreEnabled = true;
						for (const ANSICHAR* Required : RequiredExtensions)
						{
							if (!LocalEnabledExtensions.Contains(Required))
							{
								bAllRequiredAreEnabled = false;
								LocallyDisabledExtensionPlugins.Add(ExtensionPlugin);
								break;
							}
						}
						if (!bAllRequiredAreEnabled)
						{
							LocalExtensionPlugins.RemoveAt(i);
						}
					}

					// Log what we are trying now.
					if (UE_LOG_ACTIVE(LogHMD, Log))
					{
						UE_LOGF(LogHMD, Log, "Attempting to create OpenXR instance with some extensions disabled. The following extensions were enabled:");
						for (const char* Extension : EnabledExtensions)
						{
							UE_LOGF(LogHMD, Log, "- %s", Extension);
						}
						UE_LOGF(LogHMD, Log, "The following Extensions were disabled because of the ProblematicOpenXRApiLayerInfos:");
						for (int i = 0; i < NumProblematicExtensions; ++i)
						{
							if (((Bitfield >> i) & 1) == 0)
							{
								UE_LOGF(LogHMD, Log, "- %s", ProblematicExtensionList[i]);
							}
						}
						UE_LOGF(LogHMD, Log, "The following OpenXRExtensionPlugins were disabled because we disabled some or all of their required extensions:");
						for (IOpenXRExtensionPlugin* DisabledPlugin : LocallyDisabledExtensionPlugins)
						{
							UE_LOGF(LogHMD, Log, "- %ls", *DisabledPlugin->GetDisplayName());
						}
					}

					// Create!
					for (IOpenXRExtensionPlugin* Module : LocalExtensionPlugins)
					{
						LocalInfo.next = Module->OnCreateInstance(this, LocalInfo.next);
					}
					XrResult Result2 = xrCreateInstance(&LocalInfo, &Instance);
					if (XR_SUCCEEDED(Result2))
					{
						if (UE_LOG_ACTIVE(LogHMD, Log))
						{
							UE_LOGF(LogHMD, Log, "Successfully created OpenXR Instance with the following Extensions disabled because of the ProblematicOpenXRApiLayerInfos:");
							for (int i = 0; i < NumProblematicExtensions; ++i)
							{
								if (((Bitfield >> i) & 1) == 0)
								{
									UE_LOGF(LogHMD, Log, "- %ls", StringCast<TCHAR>(ProblematicExtensionList[i]).Get());
								}
							}
						}

						// Update External data, because we changed it.
						Info = LocalInfo;
						EnabledExtensions = LocalEnabledExtensions;
						ExtensionPlugins = LocalExtensionPlugins;
						return true;
					}
					else
					{
						UE_LOGF(LogHMD, Log, "Failed to create an OpenXR instance with some extensions disabled. Result is %ls.", OpenXRResultToString(Result));
					}
				} while (Bitfield > 0);					
					
				UE_LOGF(LogHMD, Log, "We did not find a combination of extensions that works using the ProblematicOpenXRApiLayerInfos.  Instance creation always failed.");
			}
		}
	}

	if (XR_FAILED(Result))
	{
		UE_LOGF(LogHMD, Warning, "Failed to create an OpenXR instance, result is %ls. Please check if you have an OpenXR runtime installed.", OpenXRResultToString(Result));
		UE_LOGF(LogHMD, Warning, "The following OpenXR API versions were enabled (see project settings:plugins:OpenXR):");
		if (bIsOpenXR1_1Enabled)
		{
			UE_LOGF(LogHMD, Warning, "- 1.1");
		}
		if (bIsOpenXR1_0Enabled)
		{
			UE_LOGF(LogHMD, Warning, "- 1.0");
		}
		UE_LOGF(LogHMD, Warning, "The following extensions were enabled:");
		for (const char* Extension : EnabledExtensions)
		{
			UE_LOGF(LogHMD, Warning, "- %s", Extension);
		}
		UE_LOGF(LogHMD, Warning, "The following layers were enumerated:");
		TArray<XrApiLayerProperties> EnumeratedLayers;
		EnumerateOpenXRApiLayers(EnumeratedLayers);
		for (XrApiLayerProperties& Layer : EnumeratedLayers)
		{
			UE_LOGF(LogHMD, Warning, "- %s", Layer.layerName);
		}

		return false;
	}

	UE_LOGF(LogHMD, Log, "xrCreateInstance created: %llu", Instance);

	return true;
}

XrSystemId FOpenXRHMDModule::GetSystemId() const
{
	XrSystemId System = XR_NULL_SYSTEM_ID;

	XrSystemGetInfo SystemInfo;
	SystemInfo.type = XR_TYPE_SYSTEM_GET_INFO;
	SystemInfo.next = nullptr;
	SystemInfo.formFactor = XR_FORM_FACTOR_HEAD_MOUNTED_DISPLAY;
	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		SystemInfo.next = Module->OnGetSystem(Instance, SystemInfo.next);
	}

	XrResult Result = xrGetSystem(Instance, &SystemInfo, &System);
	if (XR_FAILED(Result))
	{
		UE_LOGF(LogHMD, VeryVerbose, "Failed to get an OpenXR system, result is %ls", OpenXRResultToString(Result));
		return XR_NULL_SYSTEM_ID;
	}

	for (IOpenXRExtensionPlugin* Module : ExtensionPlugins)
	{
		Module->PostGetSystem(Instance, System);
	}

	return System;
}

FName FOpenXRHMDModule::ResolvePathToName(XrPath Path)
{
	{
		FReadScopeLock Lock(NameMutex);
		FName* FoundName = PathToName.Find(Path);
		if (FoundName)
		{
			// We've already previously resolved this XrPath to an FName
			return *FoundName;
		}
	}

	uint32 PathCount = 0;
	char PathChars[XR_MAX_PATH_LENGTH];
	XrResult Result = xrPathToString(Instance, Path, XR_MAX_PATH_LENGTH, &PathCount, PathChars);
	check(XR_SUCCEEDED(Result));
	if (Result == XR_SUCCESS)
	{
		// Resolve this XrPath to an FName and store it in the name map
		FName Name(PathCount - 1, PathChars);

		FWriteScopeLock Lock(NameMutex);
		PathToName.Add(Path, Name);
		NameToPath.Add(Name, Path);
		return Name;
	}
	else
	{
		return NAME_None;
	}
}

XrPath FOpenXRHMDModule::ResolveNameToPath(FName Name)
{
	{
		FReadScopeLock Lock(NameMutex);
		XrPath* FoundPath = NameToPath.Find(Name);
		if (FoundPath)
		{
			// We've already previously resolved this FName to an XrPath
			return *FoundPath;
		}
	}

	XrPath Path = XR_NULL_PATH;
	FString PathString = Name.ToString();
	XrResult Result = xrStringToPath(Instance, StringCast<ANSICHAR>(*PathString).Get(), &Path);
	check(XR_SUCCEEDED(Result));
	if (Result == XR_SUCCESS)
	{
		FWriteScopeLock Lock(NameMutex);
		PathToName.Add(Path, Name);
		NameToPath.Add(Name, Path);
		return Path;
	}
	else
	{
		return XR_NULL_PATH;
	}
}

FString FOpenXRHMDModule::GetAudioInputDevice()
{
	return OculusAudioInputDevice;
}

FString FOpenXRHMDModule::GetAudioOutputDevice()
{
	return OculusAudioOutputDevice;
}
