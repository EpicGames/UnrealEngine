// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOpenXRHMDModule.h"
#include "IHeadMountedDisplayModule.h"
#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include <openxr/openxr.h>

enum class EOpenXRAPIVersion : uint32;

class FOpenXRHMDModule : public IOpenXRHMDModule
{
public:
	FOpenXRHMDModule();
	~FOpenXRHMDModule();

	/** IHeadMountedDisplayModule implementation */
	virtual TSharedPtr< class IXRTrackingSystem, ESPMode::ThreadSafe > CreateTrackingSystem() override;
	virtual TSharedPtr< class IHeadMountedDisplayVulkanExtensions, ESPMode::ThreadSafe > GetVulkanExtensions() override;
	virtual uint64 GetGraphicsAdapterLuid() override;
	virtual FString GetAudioInputDevice() override;
	virtual FString GetAudioOutputDevice() override;

	FString GetModuleKeyName() const override
	{
		return FString(TEXT("OpenXRHMD"));
	}

	void GetModuleAliases(TArray<FString>& AliasesOut) const override
	{
		AliasesOut.Add(TEXT("OpenXR"));
	}

	virtual bool PreInit() override;
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual bool IsHMDConnected() override { return true; }
	virtual FString GetDeviceSystemName() override;
	virtual bool IsStandaloneStereoOnlyDevice() override;

	/** IOpenXRHMDModule implementation */
	virtual bool IsExtensionAvailable(const FString& Name) const override { return AvailableExtensions.Contains(Name); }
	virtual bool IsExtensionEnabled(const FString& Name) const override { return EnabledExtensions.Contains(Name); }
	virtual bool IsLayerAvailable(const FString& Name) const override { return EnabledLayers.Contains(Name); }
	virtual bool IsLayerEnabled(const FString& Name) const override { return EnabledLayers.Contains(Name); }
	virtual XrInstance GetInstance() const override { return Instance; }
	virtual XrSystemId GetSystemId() const override;

	virtual FName ResolvePathToName(XrPath Path) override;
	virtual XrPath ResolveNameToPath(FName Name) override;

private:
	void* LoaderHandle;
	XrInstance Instance;
	EOpenXRAPIVersion InstanceOpenXRAPIVersion;
	TSet<FString> AvailableExtensions;
	TSet<FString> AvailableLayers;
	TArray<const char*> EnabledExtensions;
	TArray<const char*> EnabledLayers;
	TArray<class IOpenXRExtensionPlugin*> ExtensionPlugins;
	TRefCountPtr<class FOpenXRRenderBridge> RenderBridge;
	TSharedPtr< IHeadMountedDisplayVulkanExtensions, ESPMode::ThreadSafe > VulkanExtensions;

	// We cache all attempts to convert between XrPath and FName to avoid costly resolves
	FRWLock NameMutex;
	TSortedMap<XrPath, FName> PathToName;
	TSortedMap<FName, XrPath, FDefaultAllocator, FNameFastLess> NameToPath;

	// Cache off Oculus Audio devices on PreInit so that the XrInstance can be released before Initialize
	FString OculusAudioInputDevice;
	FString OculusAudioOutputDevice;

	bool EnumerateExtensions();
	bool EnumerateLayers();
	bool InitRenderBridge();
	bool InitInstance();
	bool TryCreateInstance(XrInstanceCreateInfo& Info);
	PFN_xrGetInstanceProcAddr GetDefaultLoader();
	bool GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions);
	bool GetOptionalExtensions(TArray<const ANSICHAR*>& OutExtensions);
};
