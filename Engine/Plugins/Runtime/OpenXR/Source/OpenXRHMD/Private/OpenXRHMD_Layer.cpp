// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRHMD_Layer.h"
#include "IStereoLayers.h"
#include "StereoLayerManager.h"
#include "OpenXRHMD_Swapchain.h"
#include "OpenXRCore.h"
#include "OpenXRPlatformRHI.h"

FIntRect FOpenXRLayer::GetRightViewportSize() const
{
	FBox2D Viewport(RightEye.SwapchainSize * Desc.UVRect.Min, RightEye.SwapchainSize * Desc.UVRect.Max);
	return FIntRect(Viewport.Min.IntPoint(), Viewport.Max.IntPoint());
}

FIntRect FOpenXRLayer::GetLeftViewportSize() const
{
	FBox2D Viewport(LeftEye.SwapchainSize * Desc.UVRect.Min, LeftEye.SwapchainSize * Desc.UVRect.Max);
	return FIntRect(Viewport.Min.IntPoint(), Viewport.Max.IntPoint());
}

FVector2D FOpenXRLayer::GetRightQuadSize() const
{
	if (Desc.Flags & IStereoLayers::LAYER_FLAG_QUAD_PRESERVE_TEX_RATIO)
	{
		float AspectRatio = RightEye.SwapchainSize.Y / RightEye.SwapchainSize.X;
		return FVector2D(Desc.QuadSize.X, Desc.QuadSize.X * AspectRatio);
	}
	return Desc.QuadSize;
}

FVector2D FOpenXRLayer::GetLeftQuadSize() const
{
	if (Desc.Flags & IStereoLayers::LAYER_FLAG_QUAD_PRESERVE_TEX_RATIO)
	{
		float AspectRatio = LeftEye.SwapchainSize.Y / LeftEye.SwapchainSize.X;
		return FVector2D(Desc.QuadSize.X, Desc.QuadSize.X * AspectRatio);
	}
	return Desc.QuadSize;
}
TArray<FXrCompositionLayerUnion> FOpenXRLayer::CreateOpenXRLayer(FTransform InvTrackingToWorld, float WorldToMeters, XrSpace Space, EOpenXRLayerCreationFlags CreationFlags) const
{
	TArray<FXrCompositionLayerUnion> Headers;

	const bool bNoAlpha = Desc.Flags & IStereoLayers::LAYER_FLAG_TEX_NO_ALPHA_CHANNEL;
	const bool bIsStereo = LeftEye.Texture.IsValid();
	FTransform PositionTransform = Desc.PositionType == IStereoLayers::ELayerType::WorldLocked ?
		InvTrackingToWorld : FTransform::Identity;

	if (Desc.HasShape<FQuadLayer>())
	{
		CreateOpenXRQuadLayer(bIsStereo, bNoAlpha, PositionTransform, WorldToMeters, Space, Headers);
	}
	else if (Desc.HasShape<FCylinderLayer>())
	{
		CreateOpenXRCylinderLayer(bIsStereo, bNoAlpha, PositionTransform, WorldToMeters, Space, Headers);
	}
	else if (Desc.HasShape<FEquirectLayer>())
	{
		if(EnumHasAllFlags(CreationFlags, EOpenXRLayerCreationFlags::EquirectLayer2Supported))
		{
			CreateOpenXREquirect2Layer(bIsStereo, bNoAlpha, PositionTransform, WorldToMeters, Space, Headers);
		}
		else
		{
			CreateOpenXREquirectLayer(bIsStereo, bNoAlpha, PositionTransform, WorldToMeters, Space, Headers);
		}
	}
	else if (Desc.HasShape<FCubemapLayer>())
	{
		CreateOpenXRCubemapLayer(bIsStereo, bNoAlpha, PositionTransform, WorldToMeters, Space, Headers);
	}

	return Headers;
}

void FOpenXRLayer::ApplyCompositionDepthTestLayer(TArray<FXrCompositionLayerUnion>& Headers, EOpenXRLayerCreationFlags LayerCreationFlags, TArray<XrCompositionLayerDepthTestFB>& InCompositionDepthTestLayers) const
{
	const bool bUseDepthTest = 
		EnumHasAllFlags((IStereoLayers::ELayerFlags)Desc.Flags, IStereoLayers::LAYER_FLAG_SUPPORT_DEPTH) &&
		EnumHasAllFlags(LayerCreationFlags, EOpenXRLayerCreationFlags::DepthTestSupported);

	if (bUseDepthTest)
	{
		for (FXrCompositionLayerUnion& Header : Headers)
		{
			InCompositionDepthTestLayers.AddUninitialized();
			XrCompositionLayerDepthTestFB& LayerDepthTest = InCompositionDepthTestLayers.Last();
			LayerDepthTest.type = XR_TYPE_COMPOSITION_LAYER_DEPTH_TEST_FB;
			LayerDepthTest.next = Header.Header.next;
			LayerDepthTest.depthMask = true;
			LayerDepthTest.compareOp = XR_COMPARE_OP_LESS_FB;
			Header.Header.next = &LayerDepthTest;
		}
	}
}

int32 FOpenXRLayer::GetNumHeaders() const
{
	const bool bHasRightEye = RightEye.Swapchain.IsValid(); 
	const bool bHasLeftEye = LeftEye.Swapchain.IsValid(); 
	return bHasRightEye && bHasLeftEye ? 2 : bHasRightEye || bHasLeftEye ? 1 : 0;
}

void FOpenXRLayer::CreateOpenXRCylinderLayer(bool bIsStereo, bool bNoAlpha, FTransform PositionTransform, float WorldToMeters, XrSpace Space, TArray<FXrCompositionLayerUnion>& Headers) const
{
	XrCompositionLayerCylinderKHR Cylinder = { XR_TYPE_COMPOSITION_LAYER_CYLINDER_KHR, /*next*/ nullptr };
	Cylinder.layerFlags = bNoAlpha ? 0 : XR_COMPOSITION_LAYER_UNPREMULTIPLIED_ALPHA_BIT | XR_COMPOSITION_LAYER_BLEND_TEXTURE_SOURCE_ALPHA_BIT;
	Cylinder.space = Space;
	Cylinder.subImage.imageArrayIndex = 0;
	Cylinder.pose = ToXrPose(Desc.Transform * PositionTransform, WorldToMeters);

	const FCylinderLayer& CylinderProps = Desc.GetShape<FCylinderLayer>();
	Cylinder.radius = FMath::Abs(CylinderProps.Radius / WorldToMeters);
	Cylinder.centralAngle = FMath::Min((float)(2.0f * PI), FMath::Abs(CylinderProps.OverlayArc / CylinderProps.Radius));
	Cylinder.aspectRatio = FMath::Abs(CylinderProps.OverlayArc / CylinderProps.Height);

	FXrCompositionLayerUnion LayerUnion;
	LayerUnion.Cylinder = Cylinder;

	if (RightEye.Swapchain.IsValid())
	{
		LayerUnion.Cylinder.eyeVisibility = bIsStereo ? XR_EYE_VISIBILITY_RIGHT : XR_EYE_VISIBILITY_BOTH;
		LayerUnion.Cylinder.subImage.imageRect = ToXrRect(GetRightViewportSize());
		LayerUnion.Cylinder.subImage.swapchain = static_cast<FOpenXRSwapchain*>(RightEye.Swapchain.Get())->GetHandle();
		Headers.Add(LayerUnion);
	}
	if (LeftEye.Swapchain.IsValid())
	{
		LayerUnion.Cylinder.eyeVisibility = XR_EYE_VISIBILITY_LEFT;
		LayerUnion.Cylinder.subImage.imageRect = ToXrRect(GetLeftViewportSize());
		LayerUnion.Cylinder.subImage.swapchain = static_cast<FOpenXRSwapchain*>(LeftEye.Swapchain.Get())->GetHandle();
		Headers.Add(LayerUnion);
	}
}

void FOpenXRLayer::CreateOpenXRQuadLayer(bool bIsStereo, bool bNoAlpha, FTransform PositionTransform, float WorldToMeters, XrSpace Space, TArray<FXrCompositionLayerUnion>& Headers) const
{
	XrCompositionLayerQuad Quad = { XR_TYPE_COMPOSITION_LAYER_QUAD, nullptr };
	Quad.layerFlags = bNoAlpha ? 0 : XR_COMPOSITION_LAYER_UNPREMULTIPLIED_ALPHA_BIT |
		XR_COMPOSITION_LAYER_BLEND_TEXTURE_SOURCE_ALPHA_BIT;
	Quad.space = Space;
	Quad.subImage.imageArrayIndex = 0;
	Quad.pose = ToXrPose(Desc.Transform * PositionTransform, WorldToMeters);

	// The layer pose doesn't take the transform scale into consideration, so we need to manually apply it the quad size.
	const FVector2D LayerComponentScaler(Desc.Transform.GetScale3D().Y, Desc.Transform.GetScale3D().Z);

	FXrCompositionLayerUnion LayerUnion;
	LayerUnion.Quad = Quad;
	
	// We need to copy each layer into an OpenXR swapchain so they can be displayed by the compositor
	if (RightEye.Swapchain.IsValid())
	{
		LayerUnion.Quad.eyeVisibility = bIsStereo ? XR_EYE_VISIBILITY_RIGHT : XR_EYE_VISIBILITY_BOTH;
		LayerUnion.Quad.subImage.imageRect = ToXrRect(GetRightViewportSize());
		LayerUnion.Quad.subImage.swapchain = static_cast<FOpenXRSwapchain*>(RightEye.Swapchain.Get())->GetHandle();
		LayerUnion.Quad.size = ToXrExtent2D(GetRightQuadSize() * LayerComponentScaler, WorldToMeters);
		Headers.Add(LayerUnion);
	}
	if (LeftEye.Swapchain.IsValid())
	{
		LayerUnion.Quad.eyeVisibility = XR_EYE_VISIBILITY_LEFT;
		LayerUnion.Quad.subImage.imageRect = ToXrRect(GetLeftViewportSize());
		LayerUnion.Quad.subImage.swapchain = static_cast<FOpenXRSwapchain*>(LeftEye.Swapchain.Get())->GetHandle();
		LayerUnion.Quad.size = ToXrExtent2D(GetLeftQuadSize() * LayerComponentScaler, WorldToMeters);
		Headers.Add(LayerUnion);
	}
}

void FOpenXRLayer::SetupEquirect2(FVector2D UVScale, FVector2D UVBias, FVector2D UVPosition, FVector2D UVSize, FTransform PositionTransform, float WorldToMeters, XrCompositionLayerEquirect2KHR& Equirect2) const
{
	const FVector2D AdjustedPosition = FVector2D(
		(-UVBias.X + UVPosition.X) / UVScale.X, 
		(-UVBias.Y + UVPosition.Y) / UVScale.Y);
	const FVector2D AdjustedSize = FVector2D(UVSize.X / UVScale.X, UVSize.Y / UVScale.Y);

	const float CentralHorizontalAngle = (UE_PI * 2.0f) * AdjustedSize.X;
	const float UpperVerticalAngle = (UE_PI / 2.0f) - ((1.0f - AdjustedPosition.Y - AdjustedSize.Y) * UE_PI);
	const float LowerVerticalAngle = (-UE_PI / 2.0f) + (AdjustedPosition.Y * UE_PI);

	Equirect2.centralHorizontalAngle = CentralHorizontalAngle;
	Equirect2.upperVerticalAngle = UpperVerticalAngle; 
	Equirect2.lowerVerticalAngle = LowerVerticalAngle;
	Equirect2.pose = ToXrPose(Desc.Transform * PositionTransform, WorldToMeters);
}

void FOpenXRLayer::CreateOpenXREquirect2Layer(bool bIsStereo, bool bNoAlpha, FTransform PositionTransform, float WorldToMeters, XrSpace Space, TArray<FXrCompositionLayerUnion>& Headers) const
{
	const FEquirectLayer& EquirectProps = Desc.GetShape<FEquirectLayer>();

	XrCompositionLayerEquirect2KHR Equirect2 = { XR_TYPE_COMPOSITION_LAYER_EQUIRECT2_KHR, nullptr };
	Equirect2.layerFlags = 
		bNoAlpha ? 
		0 :
		XR_COMPOSITION_LAYER_UNPREMULTIPLIED_ALPHA_BIT | XR_COMPOSITION_LAYER_BLEND_TEXTURE_SOURCE_ALPHA_BIT;
	
	Equirect2.space = Space;
	Equirect2.subImage.imageArrayIndex = 0;
	Equirect2.radius = FMath::Abs(EquirectProps.Radius / WorldToMeters);
	FXrCompositionLayerUnion LayerUnion;

	if (RightEye.Swapchain.IsValid())
	{
		const FVector2D Size = EquirectProps.RightUVRect.Max - EquirectProps.RightUVRect.Min;
		SetupEquirect2(EquirectProps.RightScale,
					   EquirectProps.RightBias,
					   EquirectProps.RightUVRect.Min,
					   Size,
					   PositionTransform,
					   WorldToMeters,
					   Equirect2);

		LayerUnion.Equirect2 = Equirect2;
		LayerUnion.Equirect2.eyeVisibility = bIsStereo ? XR_EYE_VISIBILITY_RIGHT : XR_EYE_VISIBILITY_BOTH;
		LayerUnion.Equirect2.subImage.imageRect = ToXrRect(GetRightViewportSize());
		LayerUnion.Equirect2.subImage.swapchain = static_cast<FOpenXRSwapchain*>(RightEye.Swapchain.Get())->GetHandle();
		Headers.Add(LayerUnion);
	}

	if (LeftEye.Swapchain.IsValid())
	{
		const FVector2D Size = EquirectProps.LeftUVRect.Max - EquirectProps.LeftUVRect.Min;
		SetupEquirect2(EquirectProps.LeftScale,
					   EquirectProps.LeftBias,
					   EquirectProps.LeftUVRect.Min,
					   Size,
					   PositionTransform,
					   WorldToMeters,
					   Equirect2);

		LayerUnion.Equirect2 = Equirect2;
		LayerUnion.Equirect2.eyeVisibility = XR_EYE_VISIBILITY_LEFT;
		LayerUnion.Equirect2.subImage.imageRect = ToXrRect(GetLeftViewportSize());
		LayerUnion.Equirect2.subImage.swapchain = static_cast<FOpenXRSwapchain*>(LeftEye.Swapchain.Get())->GetHandle();
		Headers.Add(LayerUnion);
	}
}

void FOpenXRLayer::CreateOpenXREquirectLayer(bool bIsStereo, bool bNoAlpha, FTransform PositionTransform, float WorldToMeters, XrSpace Space, TArray<FXrCompositionLayerUnion>& Headers) const
{
	XrCompositionLayerEquirectKHR Equirect = { XR_TYPE_COMPOSITION_LAYER_EQUIRECT_KHR, nullptr };
	Equirect.layerFlags = bNoAlpha ? 0 : XR_COMPOSITION_LAYER_UNPREMULTIPLIED_ALPHA_BIT |
		XR_COMPOSITION_LAYER_BLEND_TEXTURE_SOURCE_ALPHA_BIT;
	Equirect.space = Space;
	Equirect.subImage.imageArrayIndex = 0;
	Equirect.pose = ToXrPose(Desc.Transform * PositionTransform, WorldToMeters);

	const FEquirectLayer& EquirectProps = Desc.GetShape<FEquirectLayer>();

	// An equirect layer with a radius of 0 is an infinite sphere.
	// As of UE 5.3, equirect layers are supported only by the Oculus OpenXR runtime and 
	// only with a radius of 0. Other radius values will be ignored.
	Equirect.radius = FMath::Abs(EquirectProps.Radius / WorldToMeters);

	FXrCompositionLayerUnion LayerUnion;
	LayerUnion.Equirect = Equirect;

	// We need to copy each layer into an OpenXR swapchain so they can be displayed by the compositor
	if (RightEye.Swapchain.IsValid())
	{
		LayerUnion.Equirect.eyeVisibility = bIsStereo ? XR_EYE_VISIBILITY_RIGHT : XR_EYE_VISIBILITY_BOTH;
		LayerUnion.Equirect.subImage.imageRect = ToXrRect(GetRightViewportSize());
		LayerUnion.Equirect.subImage.swapchain = static_cast<FOpenXRSwapchain*>(RightEye.Swapchain.Get())->GetHandle();
		LayerUnion.Equirect.scale = ToXrVector2f(EquirectProps.RightScale);
		LayerUnion.Equirect.bias = ToXrVector2f(EquirectProps.RightBias);
		Headers.Add(LayerUnion);
	}
	if (LeftEye.Swapchain.IsValid())
	{
		LayerUnion.Equirect.eyeVisibility = XR_EYE_VISIBILITY_LEFT;
		LayerUnion.Equirect.subImage.imageRect = ToXrRect(GetLeftViewportSize());
		LayerUnion.Equirect.subImage.swapchain = static_cast<FOpenXRSwapchain*>(LeftEye.Swapchain.Get())->GetHandle();
		LayerUnion.Equirect.scale = ToXrVector2f(EquirectProps.LeftScale);
		LayerUnion.Equirect.bias = ToXrVector2f(EquirectProps.LeftBias);
		Headers.Add(LayerUnion);
	}
}

void FOpenXRLayer::CreateOpenXRCubemapLayer(bool bIsStereo, bool bNoAlpha, FTransform PositionTransform, float WorldToMeters, XrSpace Space, TArray<FXrCompositionLayerUnion>& Headers) const
{
	XrCompositionLayerCubeKHR Cube = { XR_TYPE_COMPOSITION_LAYER_CUBE_KHR, nullptr };
	Cube.layerFlags = bNoAlpha ? 0 : XR_COMPOSITION_LAYER_UNPREMULTIPLIED_ALPHA_BIT | XR_COMPOSITION_LAYER_BLEND_TEXTURE_SOURCE_ALPHA_BIT;
	Cube.space = Space;
	Cube.imageArrayIndex = 0;
	Cube.orientation = ToXrPose(Desc.Transform * PositionTransform, WorldToMeters).orientation;

	const FVector2D LayerComponentScaler(Desc.Transform.GetScale3D().Y, Desc.Transform.GetScale3D().Z);

	FXrCompositionLayerUnion LayerUnion;
	LayerUnion.Cube = Cube;

	if (RightEye.Swapchain.IsValid())
	{
		LayerUnion.Cube.eyeVisibility = bIsStereo ? XR_EYE_VISIBILITY_RIGHT : XR_EYE_VISIBILITY_BOTH;
		LayerUnion.Cube.swapchain = static_cast<FOpenXRSwapchain*>(RightEye.Swapchain.Get())->GetHandle();
		Headers.Add(LayerUnion);
	}
	if (LeftEye.Swapchain.IsValid())
	{
		LayerUnion.Cube.eyeVisibility = XR_EYE_VISIBILITY_LEFT;
		LayerUnion.Cube.swapchain = static_cast<FOpenXRSwapchain*>(LeftEye.Swapchain.Get())->GetHandle();
		Headers.Add(LayerUnion);
	}
}