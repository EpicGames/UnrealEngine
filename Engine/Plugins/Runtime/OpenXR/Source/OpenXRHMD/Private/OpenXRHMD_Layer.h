// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IStereoLayers.h"
#include "OpenXRCore.h"
#include "XRSwapChain.h"

union FXrCompositionLayerUnion;

struct FOpenXRLayer
{
	struct FPerEyeTextureData
	{
		FTextureRHIRef			Texture = nullptr;
		FXRSwapChainPtr			Swapchain = nullptr;
		FVector2D				SwapchainSize{};
		ETextureDimension		SwapchainDimension = ETextureDimension::Texture2D;
		bool					bStaticSwapchain = false;
		bool					bUpdateTexture = false;

		void ConfigureSwapchain(XrSession Session, class FOpenXRRenderBridge* RenderBridge, FTextureRHIRef Texture, bool bStaticSwapchain);
	};

	IStereoLayers::FLayerDesc	Desc;

	/** Texture tracking data for the right eye.*/
	FPerEyeTextureData			RightEye;

	/** Texture tracking data for the left eye, may not be present.*/
	FPerEyeTextureData			LeftEye;

	FOpenXRLayer(const IStereoLayers::FLayerDesc& InLayerDesc)
		: Desc(InLayerDesc)
	{ }

	FIntRect GetRightViewportSize() const;
	FVector2D GetRightQuadSize() const;

	FIntRect GetLeftViewportSize() const;
	FVector2D GetLeftQuadSize() const;

	TArray<FXrCompositionLayerUnion> CreateOpenXRLayer(FTransform InvTrackingToWorld, float WorldToMeters, XrSpace Space, EOpenXRLayerCreationFlags CreationFlags) const;

	void ApplyCompositionDepthTestLayer(TArray<FXrCompositionLayerUnion>& Headers, EOpenXRLayerCreationFlags CreationFlags, TArray<XrCompositionLayerDepthTestFB>& InCompositionDepthTestLayers) const;
	
	[[nodiscard]] int32 GetNumHeaders() const;
	
private:
	void CreateOpenXRQuadLayer(bool bIsStereo, bool bNoAlpha, FTransform PositionTransform, float WorldToMeters, XrSpace Space, TArray<FXrCompositionLayerUnion>& Headers) const;
	void CreateOpenXRCylinderLayer(bool bIsStereo, bool bNoAlpha, FTransform PositionTransform, float WorldToMeters, XrSpace Space, TArray<FXrCompositionLayerUnion>& Headers) const;
	void CreateOpenXREquirectLayer(bool bIsStereo, bool bNoAlpha, FTransform PositionTransform, float WorldToMeters, XrSpace Space, TArray<FXrCompositionLayerUnion>& Headers) const;
	void CreateOpenXREquirect2Layer(bool bIsStereo, bool bNoAlpha, FTransform PositionTransform, float WorldToMeters, XrSpace Space, TArray<FXrCompositionLayerUnion>& Headers) const;
	void SetupEquirect2(FVector2D Scale, FVector2D Bias, FVector2D Position, FVector2D Size, FTransform PositionTransform, float WorldToMeters, XrCompositionLayerEquirect2KHR& Equirect2) const;
	void CreateOpenXRCubemapLayer(bool bIsStereo, bool bNoAlpha, FTransform PositionTransform, float WorldToMeters, XrSpace Space, TArray<FXrCompositionLayerUnion>& Headers) const;
};
