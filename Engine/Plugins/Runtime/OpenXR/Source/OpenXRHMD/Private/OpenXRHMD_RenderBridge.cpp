// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRHMD_RenderBridge.h"
#include "OpenXRHMD.h"
#include "OpenXRHMD_Swapchain.h"
#include "OpenXRCore.h"

bool FOpenXRRenderBridge::Present(FRHIViewport* Viewport, IRHICommandContext& RHICmdContext, int32& InOutSyncInterval)
{
	bool bNeedsNativePresent = true;

	if (OpenXRHMD)
	{
		HMDOnFinishRendering_RHIThread(RHICmdContext);
		bNeedsNativePresent = !OpenXRHMD->IsStandaloneStereoOnlyDevice();
	}

	InOutSyncInterval = 0; // VSync off

	return bNeedsNativePresent;
}

bool FOpenXRRenderBridge::PresentOnSubmissionThread(int32& InOutSyncInterval)
{
	bool bNeedsNativePresent = true;

	if (OpenXRHMD)
	{
		HMDOnFinishRendering_SubmissionThread();
		bNeedsNativePresent = !OpenXRHMD->IsStandaloneStereoOnlyDevice();
	}

	InOutSyncInterval = 0; // VSync off

	return bNeedsNativePresent;
}

void FOpenXRRenderBridge::HMDOnFinishRendering_RHIThread(IRHICommandContext& RHICmdContext)
{
	if (OpenXRHMD)
	{
		check(IsInRenderingThread() || IsInRHIThread());
		OpenXRHMD->OnFinishRendering(&RHICmdContext);
	}
}

void FOpenXRRenderBridge::HMDOnFinishRendering_SubmissionThread()
{
	if (OpenXRHMD)
	{
		check(!IsInRenderingThread() && !IsInRHIThread());
		check(OpenXRHMD->RuntimeRequiresRHIContext() == false);
		
		IRHICommandContext* RHIContext = nullptr;
		OpenXRHMD->OnFinishRendering(RHIContext);
	}
}

#ifdef XR_USE_GRAPHICS_API_D3D11
class FD3D11RenderBridge : public FOpenXRRenderBridge
{
public:
	FD3D11RenderBridge(XrInstance InInstance)
		: FOpenXRRenderBridge(InInstance)
		, Binding()
	{
		XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrGetD3D11GraphicsRequirementsKHR", (PFN_xrVoidFunction*)&GetD3D11GraphicsRequirementsKHR));
	}

	virtual uint64 GetGraphicsAdapterLuid(XrSystemId InSystem) override
	{
		XrGraphicsRequirementsD3D11KHR Requirements;
		Requirements.type = XR_TYPE_GRAPHICS_REQUIREMENTS_D3D11_KHR;
		Requirements.next = nullptr;
		if (XR_ENSURE(GetD3D11GraphicsRequirementsKHR(Instance, InSystem, &Requirements)))
		{
			return reinterpret_cast<uint64&>(Requirements.adapterLuid);
		}
		return 0;
	}

	virtual void* GetGraphicsBinding(XrSystemId InSystem) override
	{
		DXGI_ADAPTER_DESC AdapterDesc;
		GetID3D11DynamicRHI()->RHIGetAdapter()->GetDesc(&AdapterDesc);
		if (reinterpret_cast<uint64&>(AdapterDesc.AdapterLuid) != GetGraphicsAdapterLuid(InSystem))
		{
			return nullptr;
		}

		Binding.type = XR_TYPE_GRAPHICS_BINDING_D3D11_KHR;
		Binding.next = nullptr;
		Binding.device = GetID3D11DynamicRHI()->RHIGetDevice();
		return &Binding;
	}
	virtual FXRSwapChainPtr CreateSwapchain(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags) override final
	{
		return CreateSwapchain_D3D11(InSession, Format, OutActualFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags, ClearValueBinding, AuxiliaryCreateFlags);
	}
private:
	PFN_xrGetD3D11GraphicsRequirementsKHR GetD3D11GraphicsRequirementsKHR;
	XrGraphicsBindingD3D11KHR Binding;
};

FOpenXRRenderBridge* CreateRenderBridge_D3D11(XrInstance InInstance) { return new FD3D11RenderBridge(InInstance); }
#endif

#ifdef XR_USE_GRAPHICS_API_D3D12
class FD3D12RenderBridge : public FOpenXRRenderBridge
{
public:
	FD3D12RenderBridge(XrInstance InInstance)
		: FOpenXRRenderBridge(InInstance)
		, Binding()
	{
		XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrGetD3D12GraphicsRequirementsKHR", (PFN_xrVoidFunction*)&GetD3D12GraphicsRequirementsKHR));
	}

	virtual uint64 GetGraphicsAdapterLuid(XrSystemId InSystem) override
	{
		XrGraphicsRequirementsD3D12KHR Requirements;
		Requirements.type = XR_TYPE_GRAPHICS_REQUIREMENTS_D3D12_KHR;
		Requirements.next = nullptr;
		if (XR_ENSURE(GetD3D12GraphicsRequirementsKHR(Instance, InSystem, &Requirements)))
		{
			return reinterpret_cast<uint64&>(Requirements.adapterLuid);
		}
		return 0;
	}

	virtual void* GetGraphicsBinding(XrSystemId InSystem) override
	{
		LUID AdapterLuid = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU()->GetAdapterLuid();
		if (reinterpret_cast<uint64&>(AdapterLuid) != GetGraphicsAdapterLuid(InSystem))
		{
			return nullptr;
		}

		Binding.type = XR_TYPE_GRAPHICS_BINDING_D3D12_KHR;
		Binding.next = nullptr;
		Binding.device = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU();
		Binding.queue = GetID3D12DynamicRHI()->RHIGetCommandQueue();
		return &Binding;
	}
	virtual FXRSwapChainPtr CreateSwapchain(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags) override final
	{
		return CreateSwapchain_D3D12(InSession, Format, OutActualFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags, ClearValueBinding, AuxiliaryCreateFlags);
	}
	virtual void RunOnRHISubmissionThread(TFunction<void()>&& CodeToRun) override
	{
		ensure(IsInRenderingThread() || IsInRHIThread());
		GetID3D12DynamicRHI()->RHIRunOnQueue(ED3D12RHIRunOnQueueType::Graphics, [CodeToRun](ID3D12CommandQueue* Queue) { CodeToRun(); }, false);
	}

private:
	PFN_xrGetD3D12GraphicsRequirementsKHR GetD3D12GraphicsRequirementsKHR;
	XrGraphicsBindingD3D12KHR Binding;
};

FOpenXRRenderBridge* CreateRenderBridge_D3D12(XrInstance InInstance) { return new FD3D12RenderBridge(InInstance); }
#endif

#ifdef XR_USE_GRAPHICS_API_OPENGL
class FOpenGLRenderBridge : public FOpenXRRenderBridge
{
public:
	FOpenGLRenderBridge(XrInstance InInstance)
		: FOpenXRRenderBridge(InInstance)
		, Binding()
	{
		XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrGetOpenGLGraphicsRequirementsKHR", (PFN_xrVoidFunction*)&GetOpenGLGraphicsRequirementsKHR));
	}

	virtual void* GetGraphicsBinding(XrSystemId InSystem) override
	{
		XrGraphicsRequirementsOpenGLKHR Requirements;
		Requirements.type = XR_TYPE_GRAPHICS_REQUIREMENTS_OPENGL_KHR;
		Requirements.next = nullptr;
		Requirements.minApiVersionSupported = 0;
		Requirements.maxApiVersionSupported = 0;
		XR_ENSURE(GetOpenGLGraphicsRequirementsKHR(Instance, InSystem, &Requirements));

		IOpenGLDynamicRHI* RHI = GetIOpenGLDynamicRHI();
		XrVersion RHIVersion = XR_MAKE_VERSION(RHI->RHIGetGLMajorVersion(), RHI->RHIGetGLMinorVersion(), 0);
		if (RHIVersion < Requirements.minApiVersionSupported) //-V547
		{
			UE_LOGF(LogHMD, Error, "The OpenGL API version does not meet the minimum version required by the OpenXR runtime");
			return nullptr;
		}

		if (RHIVersion > Requirements.maxApiVersionSupported)
		{
			UE_LOGF(LogHMD, Warning, "The OpenGL API version has not been tested with the OpenXR runtime");
		}

#if PLATFORM_WINDOWS
		Binding.type = XR_TYPE_GRAPHICS_BINDING_OPENGL_WIN32_KHR;
		Binding.next = nullptr;
		Binding.hDC = wglGetCurrentDC();
		Binding.hGLRC = wglGetCurrentContext();
		return &Binding;
#else
		return nullptr;
#endif
	}
	virtual FXRSwapChainPtr CreateSwapchain(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags) override final
	{
		return CreateSwapchain_OpenGL(InSession, Format, OutActualFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags, ClearValueBinding, AuxiliaryCreateFlags);
	}
private:
	PFN_xrGetOpenGLGraphicsRequirementsKHR GetOpenGLGraphicsRequirementsKHR;
#if PLATFORM_WINDOWS
	XrGraphicsBindingOpenGLWin32KHR Binding;
#endif
};

FOpenXRRenderBridge* CreateRenderBridge_OpenGL(XrInstance InInstance) { return new FOpenGLRenderBridge(InInstance); }
#endif

#ifdef XR_USE_GRAPHICS_API_OPENGL_ES
class FOpenGLESRenderBridge : public FOpenXRRenderBridge
{
public:
	FOpenGLESRenderBridge(XrInstance InInstance)
		: FOpenXRRenderBridge(InInstance)
		, Binding()
	{
		XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrGetOpenGLESGraphicsRequirementsKHR", (PFN_xrVoidFunction*)&GetOpenGLESGraphicsRequirementsKHR));
	}

	virtual void* GetGraphicsBinding(XrSystemId InSystem) override
	{
		XrGraphicsRequirementsOpenGLESKHR Requirements;
		Requirements.type = XR_TYPE_GRAPHICS_REQUIREMENTS_OPENGL_ES_KHR;
		Requirements.next = nullptr;
		Requirements.minApiVersionSupported = 0;
		Requirements.maxApiVersionSupported = 0;
		XR_ENSURE(GetOpenGLESGraphicsRequirementsKHR(Instance, InSystem, &Requirements));

#if PLATFORM_ANDROID
		IOpenGLDynamicRHI* RHI = GetIOpenGLDynamicRHI();
		XrVersion RHIVersion = XR_MAKE_VERSION(RHI->RHIGetGLMajorVersion(), RHI->RHIGetGLMinorVersion(), 0);
		if (RHIVersion < Requirements.minApiVersionSupported)
		{
			UE_LOGF(LogHMD, Error, "The OpenGLES API version does not meet the minimum version required by the OpenXR runtime");
			return nullptr;
		}

		if (RHIVersion > Requirements.maxApiVersionSupported)
		{
			UE_LOGF(LogHMD, Warning, "The OpenGLES API version has not been tested with the OpenXR runtime");
		}

		Binding.type = XR_TYPE_GRAPHICS_BINDING_OPENGL_ES_ANDROID_KHR;
		Binding.next = nullptr;
		Binding.display = RHI->RHIGetEGLDisplay();
		Binding.config = RHI->RHIGetEGLConfig();
		Binding.context = RHI->RHIGetEGLContext();
		return &Binding;
#else
		return nullptr;
#endif
	}
	virtual FXRSwapChainPtr CreateSwapchain(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags) override final
	{
		return CreateSwapchain_OpenGLES(InSession, Format, OutActualFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags, ClearValueBinding, AuxiliaryCreateFlags);
	}
private:
	PFN_xrGetOpenGLESGraphicsRequirementsKHR GetOpenGLESGraphicsRequirementsKHR;
#if PLATFORM_ANDROID
	XrGraphicsBindingOpenGLESAndroidKHR Binding;
#endif
};

FOpenXRRenderBridge* CreateRenderBridge_OpenGLES(XrInstance InInstance) { return new FOpenGLESRenderBridge(InInstance); }
#endif

#ifdef XR_USE_GRAPHICS_API_VULKAN
class FVulkanRenderBridge : public FOpenXRRenderBridge
{
public:
	FVulkanRenderBridge(XrInstance InInstance)
		: FOpenXRRenderBridge(InInstance)
		, Binding()
	{
		XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrGetVulkanGraphicsRequirementsKHR", (PFN_xrVoidFunction*)&GetVulkanGraphicsRequirementsKHR));
		XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrGetVulkanGraphicsDeviceKHR", (PFN_xrVoidFunction*)&GetVulkanGraphicsDeviceKHR));
	}

	virtual void* GetGraphicsBinding(XrSystemId InSystem) override
	{
		XrGraphicsRequirementsVulkanKHR Requirements;
		Requirements.type = XR_TYPE_GRAPHICS_REQUIREMENTS_VULKAN_KHR;
		Requirements.next = nullptr;
		Requirements.minApiVersionSupported = 0;
		Requirements.maxApiVersionSupported = 0;
		XR_ENSURE(GetVulkanGraphicsRequirementsKHR(Instance, InSystem, &Requirements));

		IVulkanDynamicRHI* VulkanRHI = GetIVulkanDynamicRHI();
		const uint32 VulkanVersion = VulkanRHI->RHIGetVulkanVersion();

		// The extension uses the OpenXR version format instead of the Vulkan one
		XrVersion RHIVersion = XR_MAKE_VERSION(
			VK_VERSION_MAJOR(VulkanVersion),
			VK_VERSION_MINOR(VulkanVersion),
			VK_VERSION_PATCH(VulkanVersion)
		);
		if (RHIVersion < Requirements.minApiVersionSupported) //-V547
		{
			UE_LOGF(LogHMD, Fatal, "The Vulkan API version does not meet the minimum version required by the OpenXR runtime");
		}

		if (RHIVersion > Requirements.maxApiVersionSupported)
		{
			UE_LOGF(LogHMD, Warning, "The Vulkan API version has not been tested with the OpenXR runtime");
		}

		VkPhysicalDevice Gpu = nullptr;
		XR_ENSURE(GetVulkanGraphicsDeviceKHR(Instance, InSystem, VulkanRHI->RHIGetVkInstance(), &Gpu));
		if (Gpu != VulkanRHI->RHIGetVkPhysicalDevice())
		{
			return nullptr;
		}

		Binding.type = XR_TYPE_GRAPHICS_BINDING_VULKAN_KHR;
		Binding.next = nullptr;
		Binding.instance = VulkanRHI->RHIGetVkInstance();
		Binding.physicalDevice = VulkanRHI->RHIGetVkPhysicalDevice();
		Binding.device = VulkanRHI->RHIGetVkDevice();
		Binding.queueFamilyIndex = VulkanRHI->RHIGetGraphicsQueueFamilyIndex();
		Binding.queueIndex = 0;
		return &Binding;
	}

	virtual uint64 GetGraphicsAdapterLuid(XrSystemId InSystem) override
	{
		VkPhysicalDevice Gpu = nullptr;
		XR_ENSURE(GetVulkanGraphicsDeviceKHR(Instance, InSystem, GetIVulkanDynamicRHI()->RHIGetVkInstance(), &Gpu));
		return GetIVulkanDynamicRHI()->RHIGetGraphicsAdapterLUID(Gpu);
	}
	virtual FXRSwapChainPtr CreateSwapchain(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags) override final
	{
		return CreateSwapchain_Vulkan(InSession, Format, OutActualFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags, ClearValueBinding, AuxiliaryCreateFlags);
	}
	virtual void RunOnRHISubmissionThread(TFunction<void()>&& CodeToRun) override
	{
		ensure(IsInRenderingThread() || IsInRHIThread());
		GetIVulkanDynamicRHI()->RHIRunOnQueue(EVulkanRHIRunOnQueueType::Graphics, [CodeToRun](VkQueue Queue) { CodeToRun(); }, false);
	}

private:
	PFN_xrGetVulkanGraphicsRequirementsKHR GetVulkanGraphicsRequirementsKHR;
	PFN_xrGetVulkanGraphicsDeviceKHR GetVulkanGraphicsDeviceKHR;
	XrGraphicsBindingVulkanKHR Binding;
};

FOpenXRRenderBridge* CreateRenderBridge_Vulkan(XrInstance InInstance) { return new FVulkanRenderBridge(InInstance); }
#endif
