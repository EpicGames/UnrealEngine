// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRHMD_Swapchain.h"
#include "OpenXRCore.h"
#include "Epic_openxr.h"

static TAutoConsoleVariable<int32> CVarOpenXRSwapchainRetryCount(
	TEXT("vr.OpenXRSwapchainRetryCount"),
	9,
	TEXT("Number of times the OpenXR plugin will attempt to wait for the next swapchain image."),
	ECVF_RenderThreadSafe);

FOpenXRSwapchain::FOpenXRSwapchain(TArray<FTextureRHIRef>&& InRHITextureSwapChain, const FTextureRHIRef & InRHITexture, XrSwapchain InHandle) :
	FXRSwapChain(MoveTemp(InRHITextureSwapChain), InRHITexture),
	Handle(InHandle),
	ImageAcquired(false),
	ImageReady(false)
{
}

FOpenXRSwapchain::~FOpenXRSwapchain() 
{
	XR_ENSURE(xrDestroySwapchain(Handle));
}

// FIXME: The Vulkan extension requires access to the VkQueue in xrAcquireSwapchainImage,
// so calling this function on any other thread than the RHI thread is unsafe on some runtimes.
void FOpenXRSwapchain::IncrementSwapChainIndex_RHIThread()
{
	bool WasAcquired = false;
	ImageAcquired.compare_exchange_strong(WasAcquired, true);
	if (WasAcquired)
	{
		UE_LOGF(LogHMD, Verbose, "Attempted to redundantly acquire image %d in swapchain %p", SwapChainIndex_RHIThread.load(), reinterpret_cast<const void*>(Handle));
		return;
	}

	SCOPED_NAMED_EVENT(AcquireImage, FColor::Red);

	uint32_t SwapChainIndex = 0;
	XrSwapchainImageAcquireInfo Info;
	Info.type = XR_TYPE_SWAPCHAIN_IMAGE_ACQUIRE_INFO;
	Info.next = nullptr;
	XR_ENSURE(xrAcquireSwapchainImage(Handle, &Info, &SwapChainIndex));

	RHITexture = RHITextureSwapChain[SwapChainIndex];
	SwapChainIndex_RHIThread = SwapChainIndex;
	
	UE_LOGF(LogHMD, VeryVerbose, "FOpenXRSwapchain::IncrementSwapChainIndex_RHIThread() Acquired image %d in swapchain %p metal texture: 0x%llx", SwapChainIndex, reinterpret_cast<const void*>(Handle), RHITexture.GetReference()->GetNativeResource());
}

void FOpenXRSwapchain::WaitCurrentImage_RHIThread(int64 Timeout)
{
	bool WasAcquired = true;
	ImageAcquired.compare_exchange_strong(WasAcquired, false);
	if (!WasAcquired)
	{
		UE_LOGF(LogHMD, Warning, "Attempted to wait on unacquired image %d in swapchain %p", SwapChainIndex_RHIThread.load(), reinterpret_cast<const void*>(Handle));
		return;
	}

	bool WasReady = false;
	ImageReady.compare_exchange_strong(WasReady, true);
	if (WasReady)
	{
		UE_LOGF(LogHMD, Verbose, "Attempted to redundantly wait on image %d in swapchain %p", SwapChainIndex_RHIThread.load(), reinterpret_cast<const void*>(Handle));
		return;
	}

	SCOPED_NAMED_EVENT(WaitImage, FColor::Red);

	XrSwapchainImageWaitInfo WaitInfo;
	WaitInfo.type = XR_TYPE_SWAPCHAIN_IMAGE_WAIT_INFO;
	WaitInfo.next = nullptr;
	WaitInfo.timeout = Timeout;

	XrResult WaitResult = XR_SUCCESS;
	int RetryCount = CVarOpenXRSwapchainRetryCount.GetValueOnAnyThread();
	do
	{
		XR_ENSURE(WaitResult = xrWaitSwapchainImage(Handle, &WaitInfo));
		if (WaitResult == XR_TIMEOUT_EXPIRED) //-V547
		{
			UE_LOGF(LogHMD, Warning, "Timed out waiting on swapchain image %u! Attempts remaining %d.", SwapChainIndex_RHIThread.load(), RetryCount);
		}
	} while (WaitResult == XR_TIMEOUT_EXPIRED && RetryCount-- > 0);

	if (WaitResult != XR_SUCCESS) //-V547
	{
		// We can't continue without acquiring a new swapchain image since we won't have an image available to render to.
		UE_LOGF(LogHMD, Fatal, "Failed to wait on acquired swapchain image. This usually indicates a problem with the OpenXR runtime.");
	}
	
	UE_LOGF(LogHMD, VeryVerbose, "FOpenXRSwapchain::WaitCurrentImage_RHIThread() Waited on image swapchain %p", reinterpret_cast<const void*>(Handle));
}

void FOpenXRSwapchain::ReleaseCurrentImage_RHIThread(IRHICommandContext* RHICmdContext)
{
	bool WasReady = true;
	ImageReady.compare_exchange_strong(WasReady, false);
	if (!WasReady)
	{
		UE_LOGF(LogHMD, Warning, "Attempted to release image %d in swapchain %p that wasn't ready for being written to.", SwapChainIndex_RHIThread.load(), reinterpret_cast<const void*>(Handle));
		return;
	}

	SCOPED_NAMED_EVENT(ReleaseImage, FColor::Red);

	void* Next = nullptr;
	XrRHIContextEPIC RHIContextEPIC = { (XrStructureType)XR_TYPE_RHI_CONTEXT_EPIC };
	if (RHICmdContext != nullptr)
	{
		RHIContextEPIC.RHIContext = RHICmdContext;
		RHIContextEPIC.next = Next;
		Next = &RHIContextEPIC;
	}

	XrSwapchainImageReleaseInfo ReleaseInfo;
	ReleaseInfo.type = XR_TYPE_SWAPCHAIN_IMAGE_RELEASE_INFO;
	ReleaseInfo.next = Next;
	XR_ENSURE(xrReleaseSwapchainImage(Handle, &ReleaseInfo));

	UE_LOGF(LogHMD, VeryVerbose, "FOpenXRSwapchain::ReleaseCurrentImage_RHIThread() Released on image in swapchain %p", reinterpret_cast<const void*>(Handle));
}

uint8 FOpenXRSwapchain::GetNearestSupportedSwapchainFormat(XrSession InSession, uint8 RequestedFormat, TFunction<uint32(uint8)> ToPlatformFormat /*= nullptr*/)
{
	if (!ToPlatformFormat)
	{
		ToPlatformFormat = [](uint8 InFormat) { return GPixelFormats[InFormat].PlatformFormat; };
	}

	uint32_t FormatsCount = 0;
	XR_ENSURE(xrEnumerateSwapchainFormats(InSession, 0, &FormatsCount, nullptr));

	TArray<int64_t> Formats;
	Formats.SetNum(FormatsCount);
	XR_ENSURE(xrEnumerateSwapchainFormats(InSession, (uint32_t)Formats.Num(), &FormatsCount, Formats.GetData()));
	ensure(FormatsCount == Formats.Num());

	// Return immediately if the runtime supports the exact format being requested.
	uint32 PlatformFormat = ToPlatformFormat(RequestedFormat);
	if (Formats.Contains(PlatformFormat))
	{
		return RequestedFormat;
	}

	// Search for a fallback format in order of preference (first element in the array has the highest preference).
	uint8 FallbackFormat = 0;
	uint32 FallbackPlatformFormat = 0;
	for (int64_t Format : Formats) //-V1078
	{
		if (RequestedFormat == PF_DepthStencil)
		{
			if (Format == ToPlatformFormat(PF_D24))
			{
				FallbackFormat = PF_D24;
				FallbackPlatformFormat = Format;
				break;
			}
		}
		else
		{
			if (Format == ToPlatformFormat(PF_B8G8R8A8))
			{
				FallbackFormat = PF_B8G8R8A8;
				FallbackPlatformFormat = Format;
				break;
			}
			else if (Format == ToPlatformFormat(PF_R8G8B8A8))
			{
				FallbackFormat = PF_R8G8B8A8;
				FallbackPlatformFormat = Format;
				break;
			}
		}
	}

	if (!FallbackFormat)
	{
		UE_LOGF(LogHMD, Warning, "No compatible swapchain format found!");
		return PF_Unknown;
	}

	UE_LOGF(LogHMD, Warning, "Swapchain format not supported (%d), falling back to runtime preferred format (%d).", PlatformFormat, FallbackPlatformFormat);
	return FallbackFormat;
}
XrSwapchain FOpenXRSwapchain::CreateSwapchain(XrSession InSession, uint32 PlatformFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, void* Next /*= nullptr*/)
{
	XrSwapchainUsageFlags Usage = 0;
	if (!(CreateFlags & TexCreate_SRGB))
	{
		// On Windows both sRGB and non-sRGB integer formats have gamma correction, so since we
		// do gamma correction ourselves in the post-processor we allocate a non-SRGB format.
		// On OpenXR non-sRGB formats are assumed to be linear without gamma correction,
		// so we always allocate an sRGB swapchain format. Thus we need to specify the
		// mutable flag so we can output gamma corrected colors into an sRGB swapchain without
		// the implicit gamma correction. On mobile platforms the TexCreate_SRGB flag is specified
		// which indicates the post-processor is disabled and we do need implicit gamma correction.
		// We skip setting this flag on those platforms as it would incur a large performance hit.
		Usage |= XR_SWAPCHAIN_USAGE_MUTABLE_FORMAT_BIT;
	}
	if (EnumHasAnyFlags(CreateFlags, TexCreate_RenderTargetable))
	{
		Usage |= XR_SWAPCHAIN_USAGE_COLOR_ATTACHMENT_BIT;
	}
	if (EnumHasAnyFlags(CreateFlags, TexCreate_DepthStencilTargetable))
	{
		Usage |= XR_SWAPCHAIN_USAGE_DEPTH_STENCIL_ATTACHMENT_BIT;
	}
	if (EnumHasAnyFlags(CreateFlags, TexCreate_ShaderResource))
	{
		Usage |= XR_SWAPCHAIN_USAGE_SAMPLED_BIT;
	}
	if (EnumHasAnyFlags(CreateFlags, TexCreate_UAV))
	{
		Usage |= XR_SWAPCHAIN_USAGE_UNORDERED_ACCESS_BIT;
	}
	if (EnumHasAnyFlags(CreateFlags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable))
	{
		Usage |= XR_SWAPCHAIN_USAGE_TRANSFER_DST_BIT;
	}

	XrSwapchain Swapchain;
	XrSwapchainCreateInfo info;
	info.type = XR_TYPE_SWAPCHAIN_CREATE_INFO;
	info.next = Next;
	info.createFlags = EnumHasAnyFlags(CreateFlags, TexCreate_Dynamic) ? 0 : XR_SWAPCHAIN_CREATE_STATIC_IMAGE_BIT;
	info.usageFlags = Usage;
	info.format = PlatformFormat;
	info.sampleCount = NumSamples;
	info.width = SizeX;
	info.height = SizeY;
	info.faceCount = (Dimension == ETextureDimension::TextureCube) ? 6 : 1;
	info.arraySize = ArraySize;
	info.mipCount = NumMips;
	if (!XR_ENSURE(xrCreateSwapchain(InSession, &info, &Swapchain)))
	{
		return XR_NULL_HANDLE;
	}
	return Swapchain;
}

// TexCreate_Dynamic flag is being used in the function above to determine whether to set XR_SWAPCHAIN_CREATE_STATIC_IMAGE_BIT.  Originally,
// the Dynamic flag was non-functional in RHI, but now is used to specify whether a texture is expected to be frequently updated by the CPU.
// This flag doesn't make sense for textures created in this file, which aren't updated by the CPU, but external client code still needs to
// set the flag, so we sanitize the flag out of any RHI textures we create.
//
// TODO:  In the future, it would be cleaner to specify the static swapchain flag separately, rather than overloading TexCreate_Dynamic, but
//        that requires deprecating public APIs.  And there's a risk of clients leaving the flag set anyway when updating their code, so it
//        seems reasonable to continue to remove the flag going forward.
static ETextureCreateFlags SanitizeTextureCreateFlags(ETextureCreateFlags Flags)
{
	EnumRemoveFlags(Flags, TexCreate_Dynamic);
	return Flags;
}

template<typename T>
TArray<T> EnumerateImages(XrSwapchain InSwapchain, XrStructureType InType, void* Next = nullptr)
{
	TArray<T> Images;
	uint32_t ChainCount = 0;
	if (!XR_ENSURE(xrEnumerateSwapchainImages(InSwapchain, 0, &ChainCount, nullptr)) || ChainCount == 0)
	{
		return Images;
	}
	Images.AddZeroed(ChainCount);
	for (auto& Image : Images)
	{
		Image.type = InType;
		Image.next = Next;
	}
	if (!XR_ENSURE(xrEnumerateSwapchainImages(InSwapchain, ChainCount, &ChainCount, reinterpret_cast<XrSwapchainImageBaseHeader*>(Images.GetData()))))
	{
		Images.Empty();
	}
	return Images;
}

void FOpenXRSwapchain::GetFragmentDensityMaps(TArray<FTextureRHIRef>& OutTextureChain, const bool bIsMobileMultiViewEnabled)
{
#ifdef XR_USE_GRAPHICS_API_VULKAN
	IVulkanDynamicRHI* VulkanRHI = GetIVulkanDynamicRHI();

	XrSwapchainImageFoveationVulkanFB FoveationImage{ XR_TYPE_SWAPCHAIN_IMAGE_FOVEATION_VULKAN_FB };
	FoveationImage.next = nullptr;
	void* Next = &FoveationImage;

	XrSwapchain Swapchain = GetHandle();

	TArray<XrSwapchainImageVulkanKHR> Images = EnumerateImages<XrSwapchainImageVulkanKHR>(Swapchain, XR_TYPE_SWAPCHAIN_IMAGE_VULKAN_KHR, Next);

	if(!Images.IsEmpty())
	{
		OutTextureChain.Reset(Images.Num());
	}

	for (const XrSwapchainImageVulkanKHR& Image : Images)
	{
		const XrBaseOutStructure* NextHeader = reinterpret_cast<const XrBaseOutStructure*>(Image.next);
		const XrSwapchainImageFoveationVulkanFB* FoveationImageVulkan = reinterpret_cast<const XrSwapchainImageFoveationVulkanFB*>(NextHeader);
		OutTextureChain.Add(static_cast<FTextureRHIRef>(bIsMobileMultiViewEnabled ?
			VulkanRHI->RHICreateTexture2DArrayFromResource(GPixelFormats[PF_R8G8].UnrealFormat, FoveationImageVulkan->width, FoveationImageVulkan->height, 2, 1, 1, FoveationImageVulkan->image, TexCreate_Foveation, FClearValueBinding::White) :
			VulkanRHI->RHICreateTexture2DFromResource(GPixelFormats[PF_R8G8].UnrealFormat, FoveationImageVulkan->width, FoveationImageVulkan->height, 1, 1, FoveationImageVulkan->image, TexCreate_Foveation, FClearValueBinding::White)
			));
	}
#else
	OutTextureChain.Empty();
#endif
}


#ifdef XR_USE_GRAPHICS_API_D3D11
FXRSwapChainPtr CreateSwapchain_D3D11(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags)
{
	TFunction<uint32(uint8)> ToPlatformFormat = [](uint8 InFormat)
	{
		return GetID3D11DynamicRHI()->RHIGetSwapChainFormat(static_cast<EPixelFormat>(InFormat));
	};

	Format = FOpenXRSwapchain::GetNearestSupportedSwapchainFormat(InSession, Format, ToPlatformFormat);
	if (!Format)
	{
		return nullptr;
	}

	OutActualFormat = Format;
	XrSwapchain Swapchain = FOpenXRSwapchain::CreateSwapchain(InSession, ToPlatformFormat(Format), SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags);
	if (!Swapchain)
	{
		return nullptr;
	}

	ID3D11DynamicRHI* D3D11RHI = GetID3D11DynamicRHI();

	TArray<FTextureRHIRef> TextureChain;
	TArray<XrSwapchainImageD3D11KHR> Images = EnumerateImages<XrSwapchainImageD3D11KHR>(Swapchain, XR_TYPE_SWAPCHAIN_IMAGE_D3D11_KHR);
	if (Images.IsEmpty())
	{
		UE_LOGF(LogHMD, Error, "Failed to enumerate D3D11 swapchain images");
		xrDestroySwapchain(Swapchain);
		return nullptr;
	}
	ETextureCreateFlags SanitizedCreateFlags = SanitizeTextureCreateFlags(CreateFlags);
	for (const auto& Image : Images)
	{
		switch (Dimension)
		{
		case ETextureDimension::Texture2D:
			TextureChain.Add(D3D11RHI->RHICreateTexture2DFromResource(GPixelFormats[Format].UnrealFormat, SanitizedCreateFlags, ClearValueBinding, Image.texture));
			break;
		case ETextureDimension::Texture2DArray:
			TextureChain.Add(D3D11RHI->RHICreateTexture2DArrayFromResource(GPixelFormats[Format].UnrealFormat, SanitizedCreateFlags, ClearValueBinding, Image.texture));
			break;
		case ETextureDimension::TextureCube:
			TextureChain.Add(D3D11RHI->RHICreateTextureCubeFromResource(GPixelFormats[Format].UnrealFormat, SanitizedCreateFlags, ClearValueBinding, Image.texture));
			break;
		default:
			UE_LOGF(LogHMD, Error, "Unsupported texture dimension");
			return nullptr;
		}
	}
	return CreateXRSwapChain<FOpenXRSwapchain>(MoveTemp(TextureChain), (FTextureRHIRef&)TextureChain[0], Swapchain);
}
#endif

#ifdef XR_USE_GRAPHICS_API_D3D12
FXRSwapChainPtr CreateSwapchain_D3D12(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags)
{
	TFunction<uint32(uint8)> ToPlatformFormat = [](uint8 InFormat)
	{
		return GetID3D12DynamicRHI()->RHIGetSwapChainFormat(static_cast<EPixelFormat>(InFormat));
	};

	Format = FOpenXRSwapchain::GetNearestSupportedSwapchainFormat(InSession, Format, ToPlatformFormat);
	if (!Format)
	{
		return nullptr;
	}

	OutActualFormat = Format;
	XrSwapchain Swapchain = FOpenXRSwapchain::CreateSwapchain(InSession, ToPlatformFormat(Format), SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags);
	if (!Swapchain)
	{
		return nullptr;
	}

	ID3D12DynamicRHI* DynamicRHI = GetID3D12DynamicRHI();
	TArray<FTextureRHIRef> TextureChain;
	TArray<XrSwapchainImageD3D12KHR> Images = EnumerateImages<XrSwapchainImageD3D12KHR>(Swapchain, XR_TYPE_SWAPCHAIN_IMAGE_D3D12_KHR);
	if (Images.IsEmpty())
	{
		UE_LOGF(LogHMD, Error, "Failed to enumerate D3D12 swapchain images");
		xrDestroySwapchain(Swapchain);
		return nullptr;
	}
	ETextureCreateFlags SanitizedCreateFlags = SanitizeTextureCreateFlags(CreateFlags);
	for (const auto& Image : Images)
	{
		switch (Dimension)
		{
		case ETextureDimension::Texture2D:
			TextureChain.Add(DynamicRHI->RHICreateTexture2DFromResource(GPixelFormats[Format].UnrealFormat, SanitizedCreateFlags, ClearValueBinding, Image.texture));
			break;
		case ETextureDimension::Texture2DArray:
			TextureChain.Add(DynamicRHI->RHICreateTexture2DArrayFromResource(GPixelFormats[Format].UnrealFormat, SanitizedCreateFlags, ClearValueBinding, Image.texture));
			break;
		case ETextureDimension::TextureCube:
			TextureChain.Add(DynamicRHI->RHICreateTextureCubeFromResource(GPixelFormats[Format].UnrealFormat, SanitizedCreateFlags, ClearValueBinding, Image.texture));
			break;
		default:
			UE_LOGF(LogHMD, Error, "Unsupported texture dimension");
			ensure(false);
			return nullptr;
		}
	}
	return CreateXRSwapChain<FOpenXRSwapchain>(MoveTemp(TextureChain), (FTextureRHIRef&)TextureChain[0], Swapchain);
}
#endif

#ifdef XR_USE_GRAPHICS_API_OPENGL
FXRSwapChainPtr CreateSwapchain_OpenGL(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags)
{
	Format = FOpenXRSwapchain::GetNearestSupportedSwapchainFormat(InSession, Format);
	if (!Format)
	{
		return nullptr;
	}

	OutActualFormat = Format;
	XrSwapchain Swapchain = FOpenXRSwapchain::CreateSwapchain(InSession, GPixelFormats[Format].PlatformFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags);
	if (!Swapchain)
	{
		return nullptr;
	}

	TArray<FTextureRHIRef> TextureChain;
	IOpenGLDynamicRHI* DynamicRHI = GetIOpenGLDynamicRHI();
	TArray<XrSwapchainImageOpenGLKHR> Images = EnumerateImages<XrSwapchainImageOpenGLKHR>(Swapchain, XR_TYPE_SWAPCHAIN_IMAGE_OPENGL_KHR);
	if (Images.IsEmpty())
	{
		UE_LOGF(LogHMD, Error, "Failed to enumerate OpenGL swapchain images");
		xrDestroySwapchain(Swapchain);
		return nullptr;
	}
	ETextureCreateFlags SanitizedCreateFlags = SanitizeTextureCreateFlags(CreateFlags);
	for (const auto& Image : Images)
	{
		switch (Dimension)
		{
		case ETextureDimension::Texture2D:
			TextureChain.Add(DynamicRHI->RHICreateTexture2DFromResource(GPixelFormats[Format].UnrealFormat, SizeX, SizeY, NumMips, NumSamples, 1, ClearValueBinding, Image.image, SanitizedCreateFlags));
			break;
		case ETextureDimension::Texture2DArray:
			TextureChain.Add(DynamicRHI->RHICreateTexture2DArrayFromResource(GPixelFormats[Format].UnrealFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, 1, ClearValueBinding, Image.image, SanitizedCreateFlags));
			break;
		case ETextureDimension::TextureCube:
			TextureChain.Add(DynamicRHI->RHICreateTextureCubeFromResource(GPixelFormats[Format].UnrealFormat, SizeX, false, NumMips, 1, NumSamples, 1, ClearValueBinding, Image.image, SanitizedCreateFlags));
			break;
		default:
			UE_LOGF(LogHMD, Error, "Unsupported texture dimension");
			return nullptr;
		}
	}
	return CreateXRSwapChain<FOpenXRSwapchain>(MoveTemp(TextureChain), (FTextureRHIRef&)TextureChain[0], Swapchain);
}
#endif

#ifdef XR_USE_GRAPHICS_API_OPENGL_ES
FXRSwapChainPtr CreateSwapchain_OpenGLES(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags)
{
	Format = FOpenXRSwapchain::GetNearestSupportedSwapchainFormat(InSession, Format);
	if (!Format)
	{
		return nullptr;
	}

	OutActualFormat = Format;

	void* Next = nullptr;
	XrSwapchainCreateInfoFoveationFB FoveationCreateInfo{ XR_TYPE_SWAPCHAIN_CREATE_INFO_FOVEATION_FB };
	if (EnumHasAllFlags(AuxiliaryCreateFlags, TexCreate_Foveation))
	{
		FoveationCreateInfo.next = Next;
		FoveationCreateInfo.flags = XR_SWAPCHAIN_CREATE_FOVEATION_SCALED_BIN_BIT_FB;

		Next = &FoveationCreateInfo;
	}
	XrSwapchain Swapchain = FOpenXRSwapchain::CreateSwapchain(InSession, GPixelFormats[Format].PlatformFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags, Next);
	if (!Swapchain)
	{
		return nullptr;
	}

	TArray<FTextureRHIRef> TextureChain;
	IOpenGLDynamicRHI* DynamicRHI = GetIOpenGLDynamicRHI();
	TArray<XrSwapchainImageOpenGLESKHR> Images = EnumerateImages<XrSwapchainImageOpenGLESKHR>(Swapchain, XR_TYPE_SWAPCHAIN_IMAGE_OPENGL_ES_KHR);
	if (Images.IsEmpty())
	{
		UE_LOGF(LogHMD, Error, "Failed to enumerate OpenGL ES swapchain images");
		xrDestroySwapchain(Swapchain);
		return nullptr;
	}
	ETextureCreateFlags SanitizedCreateFlags = SanitizeTextureCreateFlags(CreateFlags);
	for (const auto& Image : Images)
	{
		switch (Dimension)
		{
		case ETextureDimension::Texture2D:
			TextureChain.Add(DynamicRHI->RHICreateTexture2DFromResource(GPixelFormats[Format].UnrealFormat, SizeX, SizeY, NumMips, NumSamples, 1, ClearValueBinding, Image.image, SanitizedCreateFlags));
			break;
		case ETextureDimension::Texture2DArray:
			TextureChain.Add(DynamicRHI->RHICreateTexture2DArrayFromResource(GPixelFormats[Format].UnrealFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, 1, ClearValueBinding, Image.image, SanitizedCreateFlags));
			break;
		case ETextureDimension::TextureCube:
			TextureChain.Add(DynamicRHI->RHICreateTextureCubeFromResource(GPixelFormats[Format].UnrealFormat, SizeX, false, NumMips, 1, NumSamples, 1, ClearValueBinding, Image.image, SanitizedCreateFlags));
			break;
		default:
			UE_LOGF(LogHMD, Error, "Unsupported texture dimension");
			return nullptr;
		}
	}
	return CreateXRSwapChain<FOpenXRSwapchain>(MoveTemp(TextureChain), (FTextureRHIRef&)TextureChain[0], Swapchain);
}
#endif

#ifdef XR_USE_GRAPHICS_API_VULKAN
FXRSwapChainPtr CreateSwapchain_Vulkan(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags)
{
	TFunction<uint32(uint8)> ToPlatformFormat = [](uint8 InFormat)
	{
		// UE renders a gamma-corrected image so we need to use an sRGB format if available
		return GetIVulkanDynamicRHI()->RHIGetSwapChainVkFormat(static_cast<EPixelFormat>(InFormat));
	};
	Format = FOpenXRSwapchain::GetNearestSupportedSwapchainFormat(InSession, Format, ToPlatformFormat);
	if (!Format)
	{
		return nullptr;
	}

	OutActualFormat = Format;
	// When we specify the mutable format flag we want to inform the runtime that we'll only use a
	// linear and an sRGB view format to allow for efficiency optimizations.
	TArray<VkFormat> ViewFormatList;
	ViewFormatList.Add((VkFormat)ToPlatformFormat(Format)); // sRGB format
	ViewFormatList.Add((VkFormat)GPixelFormats[Format].PlatformFormat); // linear format
	XrVulkanSwapchainFormatListCreateInfoKHR FormatListInfo = { XR_TYPE_VULKAN_SWAPCHAIN_FORMAT_LIST_CREATE_INFO_KHR };
	FormatListInfo.viewFormatCount = ViewFormatList.Num();
	FormatListInfo.viewFormats = ViewFormatList.GetData();

	// OpenXR wants to create sRGB swapchains. When the swapchain that is being created does not conform to this,
	// we need to tell OpenXR what formats the swapchain will be accessed in.
	// The XR_SWAPCHAIN_USAGE_MUTABLE_FORMAT_BIT will also be set in FOpenXRSwapchain::CreateSwapchain.
	// We don't need to specify additional formats if the swapchain is being created as sRGB.
	void* Next = !(CreateFlags & TexCreate_SRGB) ? &FormatListInfo : nullptr;

	XrSwapchainCreateInfoFoveationFB FoveationCreateInfo { XR_TYPE_SWAPCHAIN_CREATE_INFO_FOVEATION_FB };
	XrVulkanSwapchainCreateInfoMETA VulkanMetaCreateInfo { XR_TYPE_VULKAN_SWAPCHAIN_CREATE_INFO_META };
	if (EnumHasAllFlags(AuxiliaryCreateFlags, TexCreate_Foveation))
	{
		FoveationCreateInfo.next = Next;
		FoveationCreateInfo.flags = XR_SWAPCHAIN_CREATE_FOVEATION_FRAGMENT_DENSITY_MAP_BIT_FB;

		Next = &FoveationCreateInfo;

		if (GRHISupportsVariableRateShadingImageOffsets)
		{
			VulkanMetaCreateInfo.next = &FoveationCreateInfo;
			VulkanMetaCreateInfo.additionalUsageFlags = 0;
			VulkanMetaCreateInfo.additionalCreateFlags = VK_IMAGE_CREATE_FRAGMENT_DENSITY_MAP_OFFSET_BIT_EXT;

			Next = &VulkanMetaCreateInfo;
		}
	}
	XrSwapchain Swapchain = FOpenXRSwapchain::CreateSwapchain(InSession, ViewFormatList[0], SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags, Next);
	if (!Swapchain)
	{
		return nullptr;
	}

	IVulkanDynamicRHI* VulkanRHI = GetIVulkanDynamicRHI();

	TArray<FTextureRHIRef> TextureChain;
	TArray<XrSwapchainImageVulkanKHR> Images = EnumerateImages<XrSwapchainImageVulkanKHR>(Swapchain, XR_TYPE_SWAPCHAIN_IMAGE_VULKAN_KHR);
	if (Images.IsEmpty())
	{
		UE_LOGF(LogHMD, Error, "Failed to enumerate Vulkan swapchain images");
		xrDestroySwapchain(Swapchain);
		return nullptr;
	}
	ETextureCreateFlags SanitizedCreateFlags = SanitizeTextureCreateFlags(CreateFlags);
	for (const auto& Image : Images)
	{
		switch (Dimension)
		{
		case ETextureDimension::Texture2D:
			TextureChain.Add(VulkanRHI->RHICreateTexture2DFromResource(GPixelFormats[Format].UnrealFormat, SizeX, SizeY, NumMips, NumSamples, Image.image, SanitizedCreateFlags, ClearValueBinding));
			break;
		case ETextureDimension::Texture2DArray:
			TextureChain.Add(VulkanRHI->RHICreateTexture2DArrayFromResource(GPixelFormats[Format].UnrealFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, Image.image, SanitizedCreateFlags, ClearValueBinding));
			break;
		case ETextureDimension::TextureCube:
			TextureChain.Add(VulkanRHI->RHICreateTextureCubeFromResource(GPixelFormats[Format].UnrealFormat, SizeX, false, 1 , NumMips, Image.image, SanitizedCreateFlags, ClearValueBinding));
			break;
		default:
			return nullptr;
		}
	}
	return CreateXRSwapChain<FOpenXRSwapchain>(MoveTemp(TextureChain), (FTextureRHIRef&)TextureChain[0], Swapchain);
}
#endif
