// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeatures.h"
#include "ARTypes.h"
#include "ARTextures.h"
#include "ARTraceResult.h"
#include "DefaultSpectatorScreenController.h"
#include "UObject/SoftObjectPath.h"
#include "GenericPlatform/IInputInterface.h"
#include <openxr/openxr.h>

class FHeadMountedDisplayBase;

class IOpenXRCustomAnchorSupport
{
public:
	/**
	 * Method to add an anchor on tracking space
	 */
	virtual bool OnPinComponent(class UARPin* Pin, XrSession InSession, XrSpace TrackingSpace, XrTime DisplayTime, float worldToMeterScale) = 0;

	/**
	 * Method to remove an anchor from tracking space
	 */
	virtual void OnRemovePin(class UARPin* Pin) = 0;

	virtual void OnUpdatePin(class UARPin* Pin, XrSession InSession, XrSpace TrackingSpace, XrTime DisplayTime, float worldToMeterScale) = 0;

	// ARPin Local Store support.
	// Some Platforms/Devices have the ability to persist AR Anchors (real world positions) to the device or user account.
	// They are saved and loaded with a string identifier.

	virtual bool IsLocalPinSaveSupported() const
	{
		return false;
	}

	virtual bool ArePinsReadyToLoad()
	{
		return false;
	}

	virtual void LoadARPins(XrSession InSession, TFunction<UARPin*(FName)> OnCreatePin)
	{
	}

	virtual bool SaveARPin(XrSession InSession, FName InName, UARPin* InPin)
	{
		return false;
	}

	virtual void RemoveSavedARPin(XrSession InSession, FName InName)
	{
	}

	virtual void RemoveAllSavedARPins(XrSession InSession)
	{
	}
};

class IOpenXRCustomCaptureSupport
{
public:

	virtual bool OnGetCameraIntrinsics(struct FARCameraIntrinsics& OutCameraIntrinsics) const 
	{ 
		return false; 
	}

	/** @return the AR texture for the specified type */
	virtual class UARTexture* OnGetARTexture(EARTextureType TextureType) const
	{ 
		return nullptr; 
	}

	virtual bool OnToggleARCapture(const bool bOnOff) 
	{ 
		return false; 
	}

	virtual FTransform GetCameraTransform() const
	{ 
		return FTransform::Identity; 
	}

	virtual FVector GetWorldSpaceRayFromCameraPoint(FVector2D pixelCoordinate) const
	{
		return FVector::ZeroVector;
	}

	virtual bool IsEnabled() const
	{
		return false;
	}

	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects(const TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARCompositionComponent, const FVector Start, const FVector End, const EARLineTraceChannels TraceChannels)
	{
		return {};
	}
};

// Note: We may refactor to put OpenXRInput into the OpenXRHMD module so we can get rid of this interface.
class IOpenXRInputModule
{
public:
	virtual ~IOpenXRInputModule() {}

	virtual void OnBeginSession() = 0;
	virtual void OnDestroySession() = 0;
};

struct FInputKeyOpenXRProperties
{
	FString InputKey;
	FString InteractionProfile;
	FString OpenXRPath;
};

class IOpenXRExtensionPlugin : public IModularFeature
{
public:
	virtual ~IOpenXRExtensionPlugin(){}

	static FName GetModularFeatureName()
	{
		static FName OpenXRFeatureName = FName(TEXT("OpenXRExtension"));
		return OpenXRFeatureName;
	}

	/**
	* Register module as an extension on startup.  
	* It is common to do this in StartupModule of your IModuleInterface class (which may also be the class that implements this interface).
	* The module's LoadingPhase must be PostConfigInit or earlier because OpenXRHMD will look for these after it is loaded in that phase.
	*/
	void RegisterOpenXRExtensionModularFeature()
	{
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
	}

	void UnregisterOpenXRExtensionModularFeature()
	{
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
	}

	virtual FString GetDisplayName()
	{
		return FString(TEXT("OpenXRExtensionPlugin"));
	}

	/**
	* Optionally use a custom loader (via GetInstanceProcAddr) for the OpenXR plugin.
	*/
	virtual bool GetCustomLoader(PFN_xrGetInstanceProcAddr* OutGetProcAddr)
	{
		return false;
	}

	/**
	* Experimental: Optionally hand off the loader/plugin GetInstanceProcAddr to an extension plugin
	* to support API layering. Returns true if plugin is wrapping API. Layers can chain by using
	* received GetProcAddr to hand off API calls.
	*/
	virtual bool InsertOpenXRAPILayer(PFN_xrGetInstanceProcAddr& InOutGetProcAddr)
	{
		return false;
	}

	/**
	* Indicates that the device we're currently running does not support a spectator view.
	* This will only be called once at initialization and should only return a result based for the current device the engine is running on.
	*/
	virtual bool IsStandaloneStereoOnlyDevice()
	{
		return false;
	}
	
	/**
	* Optionally provide a custom render bridge for the OpenXR plugin.
	* Note: this returns a pointer to a new instance allocated with "new".  Calling code is responsible for eventually deleting it.
	*/
	virtual class FOpenXRRenderBridge* GetCustomRenderBridge(XrInstance InInstance)
	{
		return nullptr;
	}

	/**
	* If true pass the rhi context into some xr functions via XrRHIContextEpic.  Intended to be used where an unreal plugin wraps a XR platform api in the OpenXR api.
	*/	
	virtual bool RequiresRHIContext() const
	{
		return false;
	}


	/**
	* Fill the array with extensions required by the plugin
	* If false is returned the plugin and its extensions will be ignored
	*/
	virtual bool GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions)
	{
		return true;
	}

	/**
	* Fill the array with extensions optionally supported by the plugin
	* If false is returned the plugin and its extensions will be ignored
	*/
	virtual bool GetOptionalExtensions(TArray<const ANSICHAR*>& OutExtensions)
	{
		return true;
	}

	/**
	* Fill the array with optional extensions blocked by this plugin.
	* This can be used to disable optional extension usage by the UE OpenXR core and by other ExtensionPlugins.
	* Required extensions either from UE OpenXR core or from ExtensionPlugins are NOT blocked.  In the latter case disabling plugins may have the
	* same effect.  Otherwise code changes will be needed.
	* In general blocking optional extensions used by UE's OpenXR core should be ok because it is designed to run correctly without those extensions
	* and it seems likely the blocking plugin is replacing that functionality somehow.
	* Blocking optional extensions added by plugins is potentially more complicated and ExtensionPlugins that do this may make themselves harder to use
	* along with other ExtensionPlugins.  They are optional, so it should work, but feature use may become complex.
	* Plugin settings to explicitly disable extension use or splitting plugins up so that features can be individually enabled/disable should be considered.
	* If false is returned the returned extension list will be ignored.
	*/
	virtual bool GetBlockedOptionalExtensions(TArray<const ANSICHAR*>& OutExtensions)
	{
		return false;
	}

	/**
	* Set the output parameters to add multiple interaction profiles to OpenXR Input
	*/
	virtual bool GetInteractionProfiles(XrInstance InInstance, TArray<FString>& OutKeyPrefixes, TArray<XrPath>& OutPaths, TArray<bool>& OutHasHaptics)
	{
		return false;
	}

	/**
	 * Set the output parameter to add suggested bindings to the given interaction profile.
	 * This function gets called once for each interaction profile.
	 * If false is returned the bindings will be ignored.
	 */
	virtual bool GetSuggestedBindings(XrPath InInteractionProfile, TArray<XrActionSuggestedBinding>& OutBindings)
	{
		return false;
	}

	/**
	 * Set the output parameter to explicitly define an interaction profile and path for the given key.
	 * The same key can contain multiple entries if the key is relevant to multiple interaction profiles.
	 * If false is returned the overrides will be ignored.
	 */
	virtual bool GetInputKeyOverrides(TArray<FInputKeyOpenXRProperties>& OutOverrides)
	{
		return false;
	}

	/**
	 * Set the output parameters to provide a path to an asset in the plugin content folder that visualizes
	 * the controller in the hand represented by the user path.
	 * While it's possible to provide controller models for other interaction profiles, you should only provide
	 * controller models for the interaction profile provided by the plugin.
	 * 
	 * NOTE: All models that can be returned also need to be returned in GetControllerModels() so they're included
	 * when cooking a project. If this is skipped the controllers won't show up in packaged projects
	 */
	virtual bool GetControllerModel(XrInstance InInstance, XrPath InInteractionProfile, XrPath InDevicePath, FSoftObjectPath& OutPath)
	{
		return false;
	}

	/**
	 * Add all asset paths that need to be packaged for cooking.
	 */
	virtual void GetControllerModelsForCooking(TArray<FSoftObjectPath>& OutPaths)
	{
	}

	/**
	* Set a spectator screen controller specific to the platform
	* If true is returned and OutSpectatorScreenController is nullptr, spectator screen will be disabled
	* If false is returned a default spectator screen controller will be created
	*/
	virtual bool GetSpectatorScreenController(FHeadMountedDisplayBase* InHMDBase, TUniquePtr<FDefaultSpectatorScreenController>& OutSpectatorScreenController)
	{
		return false;
	}

	/**
	* Add any action sets provided by the plugin to be attached as active to the session
	* This allows a plugin to manage a custom actionset that will be active in xrSyncActions
	*/
	virtual void AttachActionSets(TSet<XrActionSet>& OutActionSets)
	{
	}

	/**
	* Specify action sets to be included in XrActionsSyncInfo::activeActionSets.
	*/
	virtual void GetActiveActionSetsForSync(TArray<XrActiveActionSet>& OutActiveSets)
	{
	}

	/**
	* Use this callback to handle events that the OpenXR plugin doesn't handle itself
	*/
	virtual void OnEvent(XrSession InSession, const XrEventDataBaseHeader* InHeader)
	{
	}

	/** Get custom anchor interface if provided by this extension. */
	virtual IOpenXRCustomAnchorSupport* GetCustomAnchorSupport() { return nullptr; }

	/** Get custom capture interface if provided by this extension. */
	virtual IOpenXRCustomCaptureSupport* GetCustomCaptureSupport(const EARCaptureType CaptureType) { return nullptr; }

	virtual void* OnEnumerateViewConfigurationViews(XrInstance InInstance, XrSystemId InSystem, XrViewConfigurationType InViewConfigurationType, uint32_t InViewIndex, void* InNext)
	{
		return InNext;
	}

	virtual const void* OnLocateViews(XrSession InSession, XrTime InDisplayTime, const void* InNext)
	{
		return InNext;
	}

	/**
	* Callbacks with returned pointer added to next chain, do *not* return pointers to structs on the stack.
	* Remember to assign InNext to the next pointer of your struct or otherwise you may break the next chain.
	*/

	virtual const void* OnCreateInstance(class IOpenXRHMDModule* InModule, const void* InNext)
	{
		return InNext;
	}

	virtual void PostCreateInstance(XrInstance InInstance)
	{
	}

	virtual void BindExtensionPluginDelegates(class IOpenXRExtensionPluginDelegates& OpenXRHMD)
	{
	}

	virtual const void* OnGetSystem(XrInstance InInstance, const void* InNext)
	{
		return InNext;
	}

	virtual void PostGetSystem(XrInstance InInstance, XrSystemId InSystem)
	{
	}

	virtual const void* OnCreateSession(XrInstance InInstance, XrSystemId InSystem, const void* InNext)
	{
		return InNext;
	}

	virtual void PostCreateSession(XrSession InSession)
	{
	}

	virtual const void* OnBeginSession(XrSession InSession, const void* InNext)
	{
		return InNext;
	}

	virtual void OnDestroySession(XrSession InSession)
	{
	}

	// OpenXRHMD::OnBeginSimulation_GameThread
	virtual void* OnWaitFrame(XrSession InSession, void* InNext)
	{
		return InNext;
	}

	virtual void OnBeginRendering_GameThread(XrSession InSession, FSceneViewFamily& InViewFamily, TArrayView<const uint32> VisibleLayers)
	{
	}
	
	virtual void OnBeginRendering_RenderThread_PreDeviceLocationUpdate(XrSession InSession, FRDGBuilder& GraphBuilder)
	{
	}

	virtual void OnBeginRendering_RenderThread(XrSession InSession, FRDGBuilder& GraphBuilder)
	{
	}

	virtual void PostBeginFrame_RHIThread(XrTime PredictedDisplayTime)
	{
	}

	/**
	 * Called during AllocateRenderTargetTextures (render thread) to allow the plugin to override the color swapchain pixel format.
	 * Return PF_Unknown (0) to indicate no preference and use the engine's default format selection.
	 */
	virtual const uint8 GetPreferredSwapchainFormat(uint8 RequestedFormat)
	{
		return PF_Unknown;
	}

	// OpenXRHMD::OnBeginRendering_RHIThread
	virtual const void* OnBeginFrame_RHIThread(XrSession InSession, XrTime DisplayTime, const void* InNext)	
	{
		return InNext;
	}

	virtual const void* OnBeginProjectionView(XrSession InSession, int32 InLayerIndex, int32 InViewIndex, const void* InNext)
	{
		return InNext;
	}

	virtual const void* OnBeginDepthInfo(XrSession InSession, int32 InLayerIndex, int32 InViewIndex, const void* InNext)
	{
		return InNext;
	}	
	
	// FOpenXRHMD::CreateLayer
	virtual void OnCreateLayer(uint32 LayerId)
	{
	}

	// FOpenXRHMD::DestroyLayer
	virtual void OnDestroyLayer(uint32 LayerId)
	{
	}

	// FOpenXRHMD::SetLayerDesc
	virtual void OnSetLayerDesc(uint32 LayerId)
	{
	}

	virtual void UpdateCompositionLayers_RHIThread(XrSession InSession, TArray<XrCompositionLayerBaseHeader*>& Headers)
	{
	}

	// FOpenXRHMD::OnFinishRendering_RHIThread
	virtual const void* OnEndProjectionLayer_RHIThread(XrSession InSession, int32 InLayerIndex, const void* InNext, XrCompositionLayerFlags& OutFlags)
	{
		return InNext;
	}

	// FOpenXRInputPlugin::FOpenXRInput::BuildActions
	virtual const void* OnSuggestBindings(XrPath InteractionProfile, const void* InNext)
	{
		return InNext;
	}

	// FOpenXRRenderBridge::Present, RHI thread
	virtual const void* OnEndFrame(XrSession InSession, XrTime DisplayTime, const void* InNext)
	{
		return InNext;
	}

	// FOpenXRInputPlugin::FOpenXRActionSet::FOpenXRActionSet
	virtual const void* OnCreateActionSet(XrActionSetCreateInfo InCreateInfo, const void* InNext)
	{
		return InNext;
	}

	// FOpenXRInputPlugin::FOpenXRActionSet::FOpenXRActionSet
	void PostCreateActionSet(XrActionSet InActionSet)
	{
	}

	// FOpenXRInputPlugin::FOpenXRAction::FOpenXRAction
	virtual const void* OnCreateAction(XrActionCreateInfo InCreateInfo, const void* InNext)
	{
		return InNext;
	}

	// FOpenXRInputPlugin::FOpenXRAction::FOpenXRAction
	void PostCreateAction(XrAction InAction)
	{
	}

	// FOpenXRInputPlugin::FOpenXRInput::BuildActions
	virtual const void* OnActionSetAttach(XrSessionActionSetsAttachInfo InAttachInfo, const void* InNext)
	{
		return InNext;
	}

	// FOpenXRInput::Tick, game thread, setting up for xrSyncActions.  This happens near the start of the game frame.
	virtual const void* OnSyncActions(XrSession InSession, const void* InNext)
	{
		return InNext;
	}

	// OpenXRHMD::OnStartGameFrame
	virtual void UpdateDeviceLocations(XrSession InSession, XrTime DisplayTime, XrSpace TrackingSpace)
	{
	}

	// FOpenXRInput::Tick, game thread, after xrSyncActions
	virtual void PostSyncActions(XrSession InSession)
	{
	}

	
	virtual void OnSetDeviceProperty(XrSession InSession, int32 ControllerId, const FInputDeviceProperty* Property)
	{
	}

	/** Update OpenXRHMD to use reference space types other than view, local, and stage. */
	virtual bool UseCustomReferenceSpaceType(XrReferenceSpaceType& OutReferenceSpaceType)
	{
		return false;
	}
	
	/**
	 * Start the AR system.
	 *
	 * @param SessionType The type of AR session to create
	 *
	 * @return true if the system was successfully started
	 */
	virtual void OnStartARSession(class UARSessionConfig* SessionConfig) {}

	/** Stop the AR system but leave its internal state intact. */
	virtual void OnPauseARSession() {}

	/** Stop the AR system and reset its internal state; this task must succeed. */
	virtual void OnStopARSession() {}

	virtual const void* OnCreateHandTracker(const void* InNext) { return InNext;}
	
};
