// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Skip this for now

#include "CoreMinimal.h"

struct XrSpaceLocation;

// When adding chain structs from an ExtensionPlugin it is convenient to have the calling code take ownership of the chain struct so that it can be deleted after the openxr call.
// We can also handle the linked list stuff here.
// 
// SAMPLE_USAGE in calling code (FOpenXRHMD::GetPoseForTime):
//		FOpenXRExtensionChainStructPtrs ScopedExtensionChainStructs; // Own the chain structs until we are done with them.
//		GetPoseForTimeAddChainStructsDelegate.Broadcast(&DeviceLocation, ScopedExtensionChainStructs);
//
// SAMPLE USAGE in Delegate, SpaceLocation would be the struct that is the head of the chain and we are adding an XrSpaceVelocity struct to the chain:
// void FFireAntSpaceVelocityModule::GetPoseForTimeAddChainStructs(XrSpaceLocation* SpaceLocation, FOpenXRExtensionChainStructPtrs& ChainStructPtrs)
// {
//		ChainStructPtrs.Add(MakeShared<TOpenXRExtensionChainStruct<XrSpaceVelocity>>(SpaceLocation, XR_TYPE_SPACE_VELOCITY));
// }
// OR, if you need to set data in the struct:
// void FFireAntSpaceVelocityModule::GetPoseForTimeAddChainStructs(XrSpaceLocation* SpaceLocation, FOpenXRExtensionChainStructPtrs& ChainStructPtrs)
// {
//		TSharedPtr<TOpenXRExtensionChainStruct<XrSpaceVelocity>> ChainStructPtr = MakeShared<TOpenXRExtensionChainStruct<XrSpaceVelocity>>(SpaceLocation, XR_TYPE_SPACE_VELOCITY);
//		ChainStructPtr->Data.linearVelocity = {1, 2, 3}; // Note: this is useless for XrSpaceVelocity because it is an output chain struct.
//		ChainStructPtrs.Add(ChainStructPtr);
// }

struct FOpenXRExtensionChainStruct
{
	virtual ~FOpenXRExtensionChainStruct() {};
};
typedef TSharedPtr<FOpenXRExtensionChainStruct> FOpenXRExtensionChainStructPtr;
typedef TArray<FOpenXRExtensionChainStructPtr> FOpenXRExtensionChainStructPtrs;
template <typename ChainStructType>
struct TOpenXRExtensionChainStruct : public FOpenXRExtensionChainStruct
{
	TOpenXRExtensionChainStruct(void* Head, XrStructureType XRType)
	{
		XrBaseOutStructure* HeadStructure = (XrBaseOutStructure*)Head;
		Data.type = XRType;
		Data.next = HeadStructure->next;
		HeadStructure->next = (XrBaseOutStructure*)&Data;
	}
	virtual ~TOpenXRExtensionChainStruct() {};
	ChainStructType Data;
};

// Delegates
DECLARE_MULTICAST_DELEGATE_ThreeParams(FApplyHapticFeedbackAddChainStructsDelegate, XrHapticVibration*, FOpenXRExtensionChainStructPtrs&, struct FHapticFeedbackBuffer*);

class IOpenXRExtensionPluginDelegates
{
public:
	virtual FApplyHapticFeedbackAddChainStructsDelegate& GetApplyHapticFeedbackAddChainStructsDelegate() = 0;
};

