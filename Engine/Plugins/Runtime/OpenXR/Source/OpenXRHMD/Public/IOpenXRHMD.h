// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This interface allows OpenXR specific modules to access an OpenXR specific api for OpenXRHMD, avoiding the need
// to directly access FOpenXRHMD.  
// See also IXRTrackingSystem::GetIOpenXRHMD().

#include "CoreMinimal.h"
#include "XRSwapChain.h"
#include <openxr/openxr.h>

struct FOpenXRSwapchainProperties
{
	const FStringView DebugName;
	uint8 Format;
	ETextureDimension Dimension;
	uint32 SizeX;
	uint32 SizeY;
	uint32 ArraySize;
	uint32 NumMips;
	uint32 NumSamples;
	
	ETextureCreateFlags CreateFlags;
	const FClearValueBinding& ClearValueBinding;
	ETextureCreateFlags AuxiliaryCreateFlags = ETextureCreateFlags::None;
};

enum class EOpenXRAPIVersion : uint32;

class IOpenXRHMD
{
public:
	OPENXRHMD_API virtual void SetInputModule(class IOpenXRInputModule* InInputModule) = 0;
		
	OPENXRHMD_API virtual bool IsInitialized() const = 0;
	OPENXRHMD_API virtual bool IsRunning() const = 0;
	OPENXRHMD_API virtual bool IsFocused() const = 0;

	OPENXRHMD_API virtual int32 AddTrackedDevice(XrAction Action, XrPath Path) = 0;
	OPENXRHMD_API virtual int32 AddTrackedDevice(XrAction Action, XrPath Path, XrPath SubactionPath) = 0;
	OPENXRHMD_API virtual void ResetTrackedDevices() = 0;
	OPENXRHMD_API virtual XrPath GetTrackedDevicePath(const int32 DeviceId) = 0;
	OPENXRHMD_API virtual XrSpace GetTrackedDeviceSpace(const int32 DeviceId) = 0;

	OPENXRHMD_API virtual bool IsExtensionEnabled(const FString& Name) const = 0;
	OPENXRHMD_API  virtual bool IsOpenXRAPIVersionMet(EOpenXRAPIVersion Version) const = 0;
	OPENXRHMD_API virtual XrInstance GetInstance() = 0;
	OPENXRHMD_API virtual XrSystemId GetSystem() = 0;
	OPENXRHMD_API virtual XrSession GetSession() = 0;
	OPENXRHMD_API virtual XrTime GetDisplayTime() const = 0;
	OPENXRHMD_API virtual XrSpace GetTrackingSpace() const = 0;

	OPENXRHMD_API virtual class IOpenXRExtensionPluginDelegates& GetIOpenXRExtensionPluginDelegates() = 0;

	OPENXRHMD_API virtual bool GetIsTracked(int32 DeviceId) = 0;
	OPENXRHMD_API virtual bool GetPoseForTime(int32 DeviceId, FTimespan Timespan, bool& OutTimeWasUsed, FQuat& CurrentOrientation, FVector& CurrentPosition, bool& bProvidedLinearVelocity, FVector& LinearVelocity, bool& bProvidedAngularVelocity, FVector& AngularVelocityAsAxisAndLength, bool& bProvidedLinearAcceleration, FVector& LinearAcceleration, float WorldToMetersScale) = 0;

	OPENXRHMD_API virtual TArray<class IOpenXRExtensionPlugin*>& GetExtensionPlugins() = 0;

	OPENXRHMD_API virtual void SetEnvironmentBlendMode(XrEnvironmentBlendMode NewBlendMode) = 0;
	OPENXRHMD_API virtual XrEnvironmentBlendMode GetEnvironmentBlendMode() = 0;
	OPENXRHMD_API virtual TArray<XrEnvironmentBlendMode> GetSupportedEnvironmentBlendModes() const = 0;

	OPENXRHMD_API virtual void GetFullFOVInformation(TArray<XrFovf>& FovInfos) const = 0;

	OPENXRHMD_API virtual bool AllocateSwapchainTextures_RenderThread(const FOpenXRSwapchainProperties& InSwapchainProperies, FXRSwapChainPtr& InOutSwapchain, uint8& OutActualFormat) = 0;
};

