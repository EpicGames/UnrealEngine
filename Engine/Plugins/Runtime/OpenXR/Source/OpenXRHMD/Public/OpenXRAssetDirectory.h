// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/SoftObjectPath.h"

class FOpenXRAssetDirectory
{
public: 
#if WITH_EDITORONLY_DATA
	OPENXRHMD_API static void LoadForCook();
	OPENXRHMD_API static void ReleaseAll();
#endif

	OPENXRHMD_API static FSoftObjectPath HPMixedRealityLeft;
	OPENXRHMD_API static FSoftObjectPath HPMixedRealityRight;
	OPENXRHMD_API static FSoftObjectPath HTCVive;
	OPENXRHMD_API static FSoftObjectPath HTCViveCosmosLeft;
	OPENXRHMD_API static FSoftObjectPath HTCViveCosmosRight;
	OPENXRHMD_API static FSoftObjectPath HTCViveFocus;
	OPENXRHMD_API static FSoftObjectPath HTCViveFocusPlus;
	OPENXRHMD_API static FSoftObjectPath MicrosoftMixedRealityLeft;
	OPENXRHMD_API static FSoftObjectPath MicrosoftMixedRealityRight;
	OPENXRHMD_API static FSoftObjectPath OculusGo;
	OPENXRHMD_API static FSoftObjectPath OculusTouchLeft;
	OPENXRHMD_API static FSoftObjectPath OculusTouchRight;
	OPENXRHMD_API static FSoftObjectPath OculusTouchV2Left;
	OPENXRHMD_API static FSoftObjectPath OculusTouchV2Right;
	OPENXRHMD_API static FSoftObjectPath OculusTouchV3Left;
	OPENXRHMD_API static FSoftObjectPath OculusTouchV3Right;
	OPENXRHMD_API static FSoftObjectPath PicoNeo3Left;
	OPENXRHMD_API static FSoftObjectPath PicoNeo3Right;
	OPENXRHMD_API static FSoftObjectPath SamsungGearVR;
	OPENXRHMD_API static FSoftObjectPath SamsungOdysseyLeft;
	OPENXRHMD_API static FSoftObjectPath SamsungOdysseyRight;
	OPENXRHMD_API static FSoftObjectPath ValveIndexLeft;
	OPENXRHMD_API static FSoftObjectPath ValveIndexRight;
};
