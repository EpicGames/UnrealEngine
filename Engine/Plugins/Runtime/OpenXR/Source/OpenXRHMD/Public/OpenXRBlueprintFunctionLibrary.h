// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "OpenXRCore.h"

#include "OpenXRBlueprintFunctionLibrary.generated.h"

class FOpenXRHMD;

UENUM(BlueprintType)
enum class EOpenXREnvironmentBlendMode : uint8
{
	None = 0,
	Opaque = XR_ENVIRONMENT_BLEND_MODE_OPAQUE,
	Additive = XR_ENVIRONMENT_BLEND_MODE_ADDITIVE,
    AlphaBlend = XR_ENVIRONMENT_BLEND_MODE_ALPHA_BLEND,
};

UCLASS()
class OPENXRHMD_API UOpenXRBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	 * Sets the OpenXR environment blend mode.
	 * 
	 * @param NewBlendMode		The new blend mode to be set, if supported.
	 */
	UFUNCTION(BlueprintCallable, Category="OpenXR|Passthrough")
	static void SetEnvironmentBlendMode(EOpenXREnvironmentBlendMode NewBlendMode);

	/**
	 * Gets the OpenXR environment blend mode that is currently in use.
	 */
	UFUNCTION(BlueprintCallable, Category="OpenXR|Passthrough")
	static EOpenXREnvironmentBlendMode GetEnvironmentBlendMode();

	/**
	 * Gets the OpenXR runtime's supported environment blend modes, sorted in order from highest to lowest runtime preference.
	 */
	UFUNCTION(BlueprintCallable, Category="OpenXR|Passthrough")
	static TArray<EOpenXREnvironmentBlendMode> GetSupportedEnvironmentBlendModes();

	/**
	 * Returns true if OpenXR composition layer inverted alpha is enabled.
	 */
	UFUNCTION(BlueprintCallable, Category="OpenXR|Passthrough")
	static bool IsCompositionLayerInvertedAlphaEnabled();


private:
	static FOpenXRHMD* GetOpenXRHMD();
};

