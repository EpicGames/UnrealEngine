// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include <openxr/openxr.h>
#include <openxr/openxr_reflection.h>

#define XR_ENUM_CASE_STR(name, val) case name: return TEXT(#name);
constexpr const TCHAR* OpenXRResultToString(XrResult e)
{
	switch (e)
	{
		XR_LIST_ENUM_XrResult(XR_ENUM_CASE_STR);
		default: return TEXT("Unknown");
	}
}

#define XR_SESSION_STATE_STR(name, val) case name: return TEXT(#name);
constexpr const TCHAR* OpenXRSessionStateToString(XrSessionState e)
{
	switch (e)
	{
		XR_LIST_ENUM_XrSessionState(XR_SESSION_STATE_STR);
	default: return TEXT("Unknown");
	}
}

#define XR_REFERENCE_SPACE_TYPE_STR(name, val) case name: return TEXT(#name);
constexpr const TCHAR* OpenXRReferenceSpaceTypeToString(XrReferenceSpaceType e)
{
	switch (e)
	{
		XR_LIST_ENUM_XrReferenceSpaceType(XR_REFERENCE_SPACE_TYPE_STR);
	default: return TEXT("Unknown");
	}
}

// In order to get the parameters that are in x (whatever they are) I need to implicitly capture everything.  This is quite dangerous in a lambda that is going to 
// be passed off somewhere, but this lambda is being immediately used inline.
// I find this a bit obtuse to read, so what this does:
// Define a macro with one parameter, intended to be an openxr api call and parameters, eg: "xrStringToPath(Instance, PathString, &Path)".
// Define a lamba with no parameters, implicit capture by reference, returning a bool.
// In the body of the lambda ensureMsgf with the condition of the ensure calling the function and assigning its return to XRES_Result then use the comma operator and test the result.  This means that the function is in the
// condition of the ensure, which appears in the log before the message. Because of the implicit capture the parameters of x are available.  Then we build the message of the ensure including both x and the XRES_Result. Then return the result of the ensure.
// Call this lambda immediately, thats the last two parentheses.
// Sample Usage: if (XR_ENSURE(xrStringToPath(Instance, PathString, &Path))) {...}
// Sample Usage 2: XrResult ResultForLater = XR_ERROR_VALIDATION_FAILURE; if (XR_ENSURE(ResultForLater = xrStringToPath(Instance, PathString, &Path))) {...}
// Note: in Sample Usage 2 we initialize ResultForLater and we might need to suppress static analysis with //-V547 where we use ResultForLater.
#if DO_CHECK
#define XR_ENSURE(xrCall) \
	[&] () -> bool \
	{ \
		XrResult XRES_Result; \
		return ensureMsgf((XRES_Result = xrCall, XR_SUCCEEDED(XRES_Result)), TEXT("XR call %hs failed with result: %s"), #xrCall, OpenXRResultToString(XRES_Result)); \
	} ()
#else
#define XR_ENSURE(xrCall) XR_SUCCEEDED(xrCall)
#endif

// To handle cases where we have an XrResult as a variable, and want an ensure for a function we called earlier.  Must manually provide call info text.
// Sample usage: bool Success = XR_ENSURE_RESULT(Result, "xrRequestExitSession");
#if DO_CHECK
#define XR_ENSURE_WITH_CALLINFO(xrCall, xrCallInfoText) [] (XrResult Result) \
	{ \
		return ensureMsgf(XR_SUCCEEDED(Result), TEXT("XR call %hs failed with result %s"), #xrCallInfoText, OpenXRResultToString(Result)); \
	} (xrCall)
#else
#define XR_ENSURE_WITH_CALLINFO(xrCall, xrCallInfoText) XR_SUCCEEDED(xrCall)
#endif

inline FQuat ToFQuat(XrQuaternionf Quat)
{
	return FQuat(-Quat.z, Quat.x, Quat.y, -Quat.w);
}

inline XrQuaternionf ToXrQuat(FQuat Quat)
{
	return XrQuaternionf{ (float)Quat.Y, (float)Quat.Z, -(float)Quat.X, -(float)Quat.W };
}

inline FVector ToFVector(XrVector3f Vector, float Scale = 1.0f)
{
	return FVector(-Vector.z * Scale, Vector.x * Scale, Vector.y * Scale);
}

inline FVector3f ToFVector3f(XrVector3f Vector, float Scale = 1.0f)
{
	return FVector3f(-Vector.z * Scale, Vector.x * Scale, Vector.y * Scale);
}

inline XrVector3f ToXrVector(FVector Vector, float Scale = 1.0f)
{
	if (Vector.IsZero())
		return XrVector3f{ 0.0f, 0.0f, 0.0f };

	return XrVector3f{ (float)Vector.Y / Scale, (float)Vector.Z / Scale, (float)-Vector.X / Scale };
}

inline FTransform ToFTransform(XrPosef Transform, float Scale = 1.0f)
{
	return FTransform(ToFQuat(Transform.orientation), ToFVector(Transform.position, Scale));
}

inline XrPosef ToXrPose(FTransform Transform, float Scale = 1.0f)
{
	return XrPosef{ ToXrQuat(Transform.GetRotation()), ToXrVector(Transform.GetTranslation(), Scale) };
}

inline FTimespan ToFTimespan(XrTime Time)
{
	// XrTime is a nanosecond counter, FTimespan is a 100-nanosecond counter. 
	// We are losing some precision here.
	return FTimespan((Time + 50) / 100); 
}

inline XrTime ToXrTime(FTimespan Time)
{
	return Time.GetTicks() * 100;
}

inline FIntRect ToFIntRect(XrRect2Di Rect)
{
	return FIntRect(Rect.offset.x, Rect.offset.y, Rect.offset.x + Rect.extent.width, Rect.offset.y + Rect.extent.height);
}

inline XrRect2Di ToXrRect(FIntRect Rect)
{
	return XrRect2Di{ { Rect.Min.X, Rect.Min.Y }, { Rect.Width(), Rect.Height() } };
}

inline FVector2D ToFVector2D(XrVector2f Vector, float Scale = 1.0f)
{
	return FVector2D(Vector.x * Scale, Vector.y * Scale);
}

inline XrVector2f ToXrVector2f(FVector2D Vector, float Scale = 1.0f)
{
	return XrVector2f{ (float)Vector.X / Scale, (float)Vector.Y / Scale };
}

inline FVector2D ToFVector2D(XrExtent2Df Extent, float Scale = 1.0f)
{
	return FVector2D(Extent.width * Scale, Extent.height * Scale);
}

inline XrExtent2Df ToXrExtent2D(FVector2D Vector, float Scale = 1.0f)
{
	if (Vector.IsZero())
		return XrExtent2Df{ 0.0f, 0.0f };

	return XrExtent2Df{ (float)Vector.X / Scale, (float)Vector.Y / Scale };
}

inline uint32 ToXrPriority(int32 Priority)
{
	// Ensure negative priority numbers map to the lower half of the 32-bit range.
	// We do this by casting to an unsigned int and then flipping the signed bit.
	return (uint32)Priority ^ (1 << 31);
}

union FXrCompositionLayerUnion
{
	XrCompositionLayerBaseHeader Header;
	XrCompositionLayerQuad Quad;
	XrCompositionLayerCylinderKHR Cylinder;
	XrCompositionLayerEquirectKHR Equirect;
	XrCompositionLayerEquirect2KHR Equirect2;
	XrCompositionLayerCubeKHR Cube;
};

enum class EOpenXRLayerCreationFlags : uint32
{
	None = 0u,
	EquirectLayer2Supported = (1u << 0),
	DepthTestSupported = (1u << 1),
};

ENUM_CLASS_FLAGS(EOpenXRLayerCreationFlags);

enum class EOpenXRAPIVersion : uint32
{
	V_INVALID,
	V_1_0,
	V_1_1,
};

/**
 * XrPath wrapper with convenience functions
 */
class OPENXRHMD_API FOpenXRPath
{
public:
	FOpenXRPath(XrPath InPath);

	/**
	 * Efficiently converts an FName to an XrPath
	 */
	FOpenXRPath(FName InName);

	/**
	 * Converts a string to an XrPath
	 */
	FOpenXRPath(const char* PathString);
	FOpenXRPath(const FString& PathString);

	/**
	 * Converts an XrPath to a readable format
	 *
	 * @return String representation of the path
	 */
	FString ToString() const;

	/**
	 * Get the number of characters, excluding null-terminator, that ToString() would yield
	 */
	uint32 GetStringLength() const;

	/**
	 * Efficiently converts an XrPath to an FName
	 *
	 * @return FName representing the path
	 */
	FName ToName() const;

	XrPath ToXRPath() const { return Path; }

	operator bool() const { return Path != XR_NULL_PATH; }
	operator XrPath() const { return Path; }
	operator FString() const { return ToString(); }
	operator FName() const { return ToName(); }

	/**
	 * Operators to append another path ensuring the / character is used between them
	 */
	FOpenXRPath operator/(const char* Suffix) const { return FOpenXRPath(ToString() / Suffix); }
	FOpenXRPath operator/(FString Suffix) const { return FOpenXRPath(ToString() / Suffix); }
	FOpenXRPath operator/(FOpenXRPath Suffix) const { return FOpenXRPath(ToString() / Suffix); }

private:
	XrPath Path;
};

/** List all OpenXR global entry points used by Unreal. */
#define ENUM_XR_ENTRYPOINTS_GLOBAL(EnumMacro) \
	EnumMacro(PFN_xrEnumerateApiLayerProperties,xrEnumerateApiLayerProperties) \
	EnumMacro(PFN_xrEnumerateInstanceExtensionProperties,xrEnumerateInstanceExtensionProperties) \
	EnumMacro(PFN_xrCreateInstance,xrCreateInstance)

/** List all OpenXR instance entry points used by Unreal. */
#define ENUM_XR_ENTRYPOINTS(EnumMacro) \
	EnumMacro(PFN_xrDestroyInstance,xrDestroyInstance) \
	EnumMacro(PFN_xrGetInstanceProperties,xrGetInstanceProperties) \
	EnumMacro(PFN_xrPollEvent,xrPollEvent) \
	EnumMacro(PFN_xrResultToString,xrResultToString) \
	EnumMacro(PFN_xrStructureTypeToString,xrStructureTypeToString) \
	EnumMacro(PFN_xrGetSystem,xrGetSystem) \
	EnumMacro(PFN_xrGetSystemProperties,xrGetSystemProperties) \
	EnumMacro(PFN_xrEnumerateEnvironmentBlendModes,xrEnumerateEnvironmentBlendModes) \
	EnumMacro(PFN_xrCreateSession,xrCreateSession) \
	EnumMacro(PFN_xrDestroySession,xrDestroySession) \
	EnumMacro(PFN_xrEnumerateReferenceSpaces,xrEnumerateReferenceSpaces) \
	EnumMacro(PFN_xrCreateReferenceSpace,xrCreateReferenceSpace) \
	EnumMacro(PFN_xrGetReferenceSpaceBoundsRect,xrGetReferenceSpaceBoundsRect) \
	EnumMacro(PFN_xrCreateActionSpace,xrCreateActionSpace) \
	EnumMacro(PFN_xrLocateSpace,xrLocateSpace) \
	EnumMacro(PFN_xrDestroySpace,xrDestroySpace) \
	EnumMacro(PFN_xrEnumerateViewConfigurations,xrEnumerateViewConfigurations) \
	EnumMacro(PFN_xrGetViewConfigurationProperties,xrGetViewConfigurationProperties) \
	EnumMacro(PFN_xrEnumerateViewConfigurationViews,xrEnumerateViewConfigurationViews) \
	EnumMacro(PFN_xrEnumerateSwapchainFormats,xrEnumerateSwapchainFormats) \
	EnumMacro(PFN_xrCreateSwapchain,xrCreateSwapchain) \
	EnumMacro(PFN_xrDestroySwapchain,xrDestroySwapchain) \
	EnumMacro(PFN_xrEnumerateSwapchainImages,xrEnumerateSwapchainImages) \
	EnumMacro(PFN_xrAcquireSwapchainImage,xrAcquireSwapchainImage) \
	EnumMacro(PFN_xrWaitSwapchainImage,xrWaitSwapchainImage) \
	EnumMacro(PFN_xrReleaseSwapchainImage,xrReleaseSwapchainImage) \
	EnumMacro(PFN_xrBeginSession,xrBeginSession) \
	EnumMacro(PFN_xrEndSession,xrEndSession) \
	EnumMacro(PFN_xrRequestExitSession,xrRequestExitSession) \
	EnumMacro(PFN_xrWaitFrame,xrWaitFrame) \
	EnumMacro(PFN_xrBeginFrame,xrBeginFrame) \
	EnumMacro(PFN_xrEndFrame,xrEndFrame) \
	EnumMacro(PFN_xrLocateViews,xrLocateViews) \
	EnumMacro(PFN_xrStringToPath,xrStringToPath) \
	EnumMacro(PFN_xrPathToString,xrPathToString) \
	EnumMacro(PFN_xrCreateActionSet,xrCreateActionSet) \
	EnumMacro(PFN_xrDestroyActionSet,xrDestroyActionSet) \
	EnumMacro(PFN_xrCreateAction,xrCreateAction) \
	EnumMacro(PFN_xrDestroyAction,xrDestroyAction) \
	EnumMacro(PFN_xrSuggestInteractionProfileBindings,xrSuggestInteractionProfileBindings) \
	EnumMacro(PFN_xrAttachSessionActionSets,xrAttachSessionActionSets) \
	EnumMacro(PFN_xrGetCurrentInteractionProfile,xrGetCurrentInteractionProfile) \
	EnumMacro(PFN_xrGetActionStateBoolean,xrGetActionStateBoolean) \
	EnumMacro(PFN_xrGetActionStateFloat,xrGetActionStateFloat) \
	EnumMacro(PFN_xrGetActionStateVector2f,xrGetActionStateVector2f) \
	EnumMacro(PFN_xrGetActionStatePose,xrGetActionStatePose) \
	EnumMacro(PFN_xrSyncActions,xrSyncActions) \
	EnumMacro(PFN_xrEnumerateBoundSourcesForAction,xrEnumerateBoundSourcesForAction) \
	EnumMacro(PFN_xrGetInputSourceLocalizedName,xrGetInputSourceLocalizedName) \
	EnumMacro(PFN_xrApplyHapticFeedback,xrApplyHapticFeedback) \
	EnumMacro(PFN_xrStopHapticFeedback,xrStopHapticFeedback)

/** Declare all XR functions in a namespace to avoid conflicts with the loader exported symbols. */
#define DECLARE_XR_ENTRYPOINTS(Type,Func) extern Type OPENXRHMD_API Func;
namespace OpenXRDynamicAPI
{
	ENUM_XR_ENTRYPOINTS_GLOBAL(DECLARE_XR_ENTRYPOINTS);
	ENUM_XR_ENTRYPOINTS(DECLARE_XR_ENTRYPOINTS);
	DECLARE_XR_ENTRYPOINTS(PFN_xrGetInstanceProcAddr, xrGetInstanceProcAddr)
}
using namespace OpenXRDynamicAPI;

/**
 * Initialize essential OpenXR functions.
 * @returns true if initialization was successful.
 */
bool PreInitOpenXRCore(PFN_xrGetInstanceProcAddr InGetProcAddr);

/**
 * Initialize core OpenXR functions.
 * @returns true if initialization was successful.
 */
bool InitOpenXRCore(XrInstance Instance);

void OPENXRHMD_API EnumerateOpenXRApiLayers(TArray<XrApiLayerProperties>& OutProperties);

inline void FilterActionName(const char* InActionName, char* OutActionName)
{
	static_assert(XR_MAX_ACTION_NAME_SIZE == XR_MAX_ACTION_SET_NAME_SIZE);

	// Ensure the action name is a well-formed path
	size_t i;
	for (i = 0; i < XR_MAX_ACTION_NAME_SIZE - 1 && InActionName[i] != '\0'; i++)
	{
		unsigned char c = InActionName[i];
		OutActionName[i] = (c == ' ') ? '-' : isalnum(c) ? tolower(c) : '_';
	}
	OutActionName[i] = '\0';
}

namespace OpenXR
{
	template<typename T>
	T* FindChainedStructByType(void* Head, XrStructureType XRType)
	{
		XrBaseOutStructure* Ptr = (XrBaseOutStructure*)Head;
		while (Ptr)
		{
			if (Ptr->type == XRType)
			{
				return (T*)Ptr;
			}
			else
			{
				Ptr = Ptr->next;
			}
		}
		return nullptr;
	}

	template<typename T>
	const T* FindChainedStructByType(const void* Head, XrStructureType XRType)
	{
		const XrBaseInStructure* Ptr = (XrBaseInStructure*)Head;
		while (Ptr)
		{
			if (Ptr->type == XRType)
			{
				return (const T*)Ptr;
			}
			else
			{
				Ptr = Ptr->next;
			}
		}
		return nullptr;
	}

	inline void AppendChainStruct(void*& Tail, void* NewTail)
	{
		XrBaseOutStructure*& TailXR = reinterpret_cast<XrBaseOutStructure*&>(Tail);
		XrBaseOutStructure*& NewTailXR = reinterpret_cast<XrBaseOutStructure*&>(NewTail);
		
		check(TailXR->next == nullptr);
		check(NewTailXR->next == nullptr);
		TailXR->next = NewTailXR;
		Tail = NewTailXR;
	}
}
