// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettingsBackedByCVars.h"
#include "OpenXRHMDSettings.generated.h"

/**
* Implements the settings for the OpenXR plugin.
*/
UCLASS(config=Engine, defaultconfig, meta = (DisplayName = "OpenXR"))
class OPENXRHMD_API UOpenXRHMDSettings : public UDeveloperSettingsBackedByCVars
{
	GENERATED_BODY()

public:
	/** Enables foveation provided by the XR_FB_foveation OpenXR extension. */
	UPROPERTY(config, EditAnywhere, Category = "Foveation", meta = (
		ToolTip = "Enables foveation provided by the XR_FB_foveation OpenXR extension. Requires support for hardware variable rate shading.", 
		DisplayName = "Enable XR_FB_foveation extension"))
	bool bIsFBFoveationEnabled = false;

	/** Enables alpha inversion of the background layer. */
	UPROPERTY(config, EditAnywhere, Category = "Passthrough", meta = (
		ConsoleVariable = "xr.OpenXRInvertAlpha",
		ToolTip = "Enables alpha inversion of the background layer if the XR_EXT_composition_layer_inverted_alpha extension or XR_FB_composition_layer_alpha_blend is supported.", 
		DisplayName = "Invert scene alpha for passthrough"))
	bool bOpenXRInvertAlpha = false;

	/** Enable support for OpenXR 1.0. */
	UPROPERTY(config, EditAnywhere, Category = "OpenXR Versions", meta = (
		ToolTip = "Enable support for OpenXR 1.0. If multiple versions are supported by the current OpenXR Runtime the latest version will be used.",
		DisplayName = "Enable OpenXR 1.0"))
	bool bIsOpenXR1_0Enabled = true;

	/** Enable support for OpenXR 1.1. */
	UPROPERTY(config, EditAnywhere, Category = "OpenXR Versions", meta = (
		ToolTip = "Enable support for OpenXR 1.1. If multiple versions are supported by the current OpenXR Runtime the latest version will be used.",
		DisplayName = "Enable OpenXR 1.1"))
	bool bIsOpenXR1_1Enabled = true;

	/** Extension Blocking */
	UPROPERTY(config, EditAnywhere, Category = "OpenXR", meta = (
		ToolTip = "List of OpenXR extension names that will be blocked and not used.\n The names must exactly match those provided in openxr_platform.h (eg \"XR_KHR_vulkan_enable\" from #define XR_KHR_VULKAN_ENABLE_EXTENSION_NAME \"XR_KHR_vulkan_enable\")\nIf a UE Core optional extension is blocked UE's OpenXR support should continue to function, but functionality or performance may be degraded.\nIf a UE Core required extension is blocked UE's OpenXR support will be disabled.\nIf a OpenXRExtensionPlugin optional extension is blocked the plugin should continue to function, but functionality or performance may be degraded.\nIf a OpenXRExtensionPlugin required extension is blocked that plugin will be disabled.\nAnticipated use cases:\n  Facilitation of testing without certain extensions\n  Working around runtime bugs where an extension is malfunctioning (Note that if the runtime is later fixed your project will still not use it!).\n You will need to restart the editor after changing this setting to have it effect PIE.",
		DisplayName = "Blocked Extensions"))
	TArray<FString> BlockedExtensions;

	/** Projection Layer Priority */
	UPROPERTY(config, EditAnywhere, Category = "Stereo Layers", meta = (
		ConsoleVariable = "xr.OpenXRProjectionLayerPriority",
		ToolTip = "Render priority for the projection layer (main UE environment).\nStereo layers with priorities greater than or equal to this value will render above the projection layer. Layers with priorities less than this value will render below the projection layer.\nNote that some OpenXR runtimes will ignore this setting and always render the projection layer below all stereo layers.",
		DisplayName = "Projection Layer Priority"))
	int32 Priority = 0;

	/** Sort face-locked layers above all other layers */
	UPROPERTY(config, EditAnywhere, Category = "Stereo Layers", meta = (
		ConsoleVariable = "xr.OpenXRSortFaceLockedLayersAboveOtherLayers",
		ToolTip = "Sort face locked stereo layers above all other layers, including the projection layer, even if the face locked layer has a lower priority.",
		DisplayName = "Sort Face Locked Layers Above Other Layers"))
	bool bSortFaceLockedLayersAboveOtherLayers = false;

public:
	virtual FName GetCategoryName() const override { return FName(TEXT("Plugins")); }
};
