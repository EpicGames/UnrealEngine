// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS
#define XR_USE_PLATFORM_WIN32		1
#endif

#ifdef OPENXRHMD_HAVE_DX11
#define XR_USE_GRAPHICS_API_D3D11	1
#endif

#ifdef OPENXRHMD_HAVE_DX12
#define XR_USE_GRAPHICS_API_D3D12	1
#endif

#ifdef OPENXRHMD_HAVE_OPENGL
#define XR_USE_GRAPHICS_API_OPENGL	1
#endif

#if PLATFORM_ANDROID
#define XR_USE_PLATFORM_ANDROID 1
#endif

#ifdef OPENXRHMD_HAVE_OPENGLES
#define XR_USE_GRAPHICS_API_OPENGL_ES	1
#endif

#ifdef OPENXRHMD_HAVE_VULKAN
#define XR_USE_GRAPHICS_API_VULKAN 1
#endif

//-------------------------------------------------------------------------------------------------
// D3D11
//-------------------------------------------------------------------------------------------------

#ifdef XR_USE_GRAPHICS_API_D3D11
#include "ID3D11DynamicRHI.h"
#endif // XR_USE_GRAPHICS_API_D3D11


//-------------------------------------------------------------------------------------------------
// D3D12
//-------------------------------------------------------------------------------------------------

#ifdef XR_USE_GRAPHICS_API_D3D12
#include "ID3D12DynamicRHI.h"
#endif // XR_USE_GRAPHICS_API_D3D12


//-------------------------------------------------------------------------------------------------
// OpenGL
//-------------------------------------------------------------------------------------------------

#if defined(XR_USE_GRAPHICS_API_OPENGL) || defined(XR_USE_GRAPHICS_API_OPENGL_ES)
#include "IOpenGLDynamicRHI.h"
#endif // XR_USE_GRAPHICS_API_OPENGL

//-------------------------------------------------------------------------------------------------
// Vulkan
//-------------------------------------------------------------------------------------------------

#ifdef XR_USE_GRAPHICS_API_VULKAN
#include "IVulkanDynamicRHI.h"
#endif // XR_USE_GRAPHICS_API_VULKAN

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#if PLATFORM_ANDROID
#include <android_native_app_glue.h>
#endif

#include <openxr/openxr_platform.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif
