// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
    public class OpenXRInput : ModuleRules
    {
        public OpenXRInput(ReadOnlyTargetRules Target) : base(Target)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
					"ApplicationCore",
                    "Engine",
                    "InputDevice",
                    "InputCore",
                    "HeadMountedDisplay",
					"XRBase",
                    "OpenXRHMD",
                }
                );

			PublicDependencyModuleNames.Add("EnhancedInput");

			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenXR");

            if (Target.bBuildEditor == true)
            {
				PrivateDependencyModuleNames.Add("EditorFramework");
                PrivateDependencyModuleNames.Add("UnrealEd");
				PrivateDependencyModuleNames.Add("InputEditor");
				PrivateDependencyModuleNames.Add("SourceControl");
			}
        }
    }
}
