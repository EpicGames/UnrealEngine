// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRInput.h"
#include "EnhancedInputDeveloperSettings.h"
#include "IOpenXRHMD.h"
#include "IXRTrackingSystem.h"
#include "OpenXRCore.h"
#include "UObject/UObjectIterator.h"
#include "GameFramework/InputSettings.h"
#include "IOpenXRExtensionPlugin.h"
#include "IOpenXRExtensionPluginDelegates.h"
#include "HeadMountedDisplayFunctionLibrary.h"

#include "EnhancedInputLibrary.h"
#include "EnhancedInputSubsystemInterface.h"
#include "EnhancedInputModule.h"
#include "InputAction.h"
#include "InputMappingContext.h"
#include "PlayerMappableInputConfig.h"
#include "UserSettings/EnhancedInputUserSettings.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Engine/Engine.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "Editor.h"
#include "EnhancedInputEditorSubsystem.h"
#endif

#include <openxr/openxr.h>

#define LOCTEXT_NAMESPACE "OpenXRInputPlugin"

namespace OpenXRSourceNames
{
	static const FName Head("Head");
	static const FName AnyHand("AnyHand");
	static const FName Left("Left");
	static const FName Right("Right");
	static const FName LeftGrip("LeftGrip");
	static const FName RightGrip("RightGrip");
	static const FName LeftAim("LeftAim");
	static const FName RightAim("RightAim");
	static const FName LeftPalm("LeftPalm");
	static const FName RightPalm("RightPalm");
}

FORCEINLINE XrActionType ToActionType(EInputActionValueType ValueType)
{
	switch (ValueType)
	{
		case EInputActionValueType::Boolean: return XR_ACTION_TYPE_BOOLEAN_INPUT;
		case EInputActionValueType::Axis1D: return XR_ACTION_TYPE_FLOAT_INPUT;
		case EInputActionValueType::Axis2D: return XR_ACTION_TYPE_VECTOR2F_INPUT;
		case EInputActionValueType::Axis3D:
			// TODO: Add 3D vector support to OpenXR Input
			ensure(false);
			return XR_ACTION_TYPE_VECTOR2F_INPUT;
	}
	ensure(false);
	return (XrActionType)0;
}

TSharedPtr< class IInputDevice > FOpenXRInputPlugin::CreateInputDevice(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler)
{
	if (InputDevice)
		InputDevice->SetMessageHandler(InMessageHandler);
	return InputDevice;
}

IMPLEMENT_MODULE(FOpenXRInputPlugin, OpenXRInput)

FOpenXRInputPlugin::FOpenXRInputPlugin()
	: InputDevice()
{
}

FOpenXRInputPlugin::~FOpenXRInputPlugin()
{
}

void FOpenXRInputPlugin::StartupModule()
{
	// Note: XRSystem may be null, for example in the editor.  But we still need the input device to enumerate sources.
	InputDevice = MakeShared<FOpenXRInput>(GEngine->XRSystem.Get());

	IOpenXRInputPlugin::StartupModule();
}

FOpenXRInputPlugin::FOpenXRAction::FOpenXRAction(XrActionSet InActionSet,
	XrActionType InActionType, const FName& InName, const FString& InLocalizedName,
	const TArray<XrPath>& InSubactionPaths, const TObjectPtr<const UInputAction>& InObject, IOpenXRHMD* OpenXRHMD)
	: FOpenXRAction(InActionSet, InActionType, InName, InLocalizedName, InSubactionPaths, OpenXRHMD)
{
	Object = InObject;
}

FOpenXRInputPlugin::FOpenXRAction::FOpenXRAction(XrActionSet InActionSet,
	XrActionType InActionType, const FName& InName, const FString& InLocalizedName,
	const TArray<XrPath>& InSubactionPaths, IOpenXRHMD* OpenXRHMD)
	: Set(InActionSet)
	, Type(InActionType)
	, Name(InName)
	, Handle(XR_NULL_HANDLE)
	, Object(nullptr)
{
	char ActionName[NAME_SIZE];
	Name.GetPlainANSIString(ActionName);

	XrActionCreateInfo Info;
	Info.type = XR_TYPE_ACTION_CREATE_INFO;
	Info.next = nullptr;
	FilterActionName(ActionName, Info.actionName);
	Info.actionType = Type;
	Info.countSubactionPaths = InSubactionPaths.Num();
	Info.subactionPaths = InSubactionPaths.GetData();
	if (!InLocalizedName.IsEmpty())
	{
		FTCHARToUTF8_Convert::Convert(Info.localizedActionName, XR_MAX_LOCALIZED_ACTION_NAME_SIZE, *InLocalizedName, InLocalizedName.Len() + 1);
	}
	else
	{
		FCStringAnsi::Strncpy(Info.localizedActionName, ActionName, XR_MAX_LOCALIZED_ACTION_NAME_SIZE);
	}

	if (OpenXRHMD)
	{
		for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
		{
			Info.next = Plugin->OnCreateAction(Info, Info.next);
		}
	}

	XR_ENSURE(xrCreateAction(Set, &Info, &Handle));

	if (OpenXRHMD)
	{
		for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
		{
			Plugin->PostCreateAction(Handle);
		}
	}
}

FOpenXRInputPlugin::FOpenXRActionSet::FOpenXRActionSet(XrInstance InInstance,
	const FName& InName, const FString& InLocalizedName, uint32 InPriority,
	const TObjectPtr<const UInputMappingContext>& InObject, IOpenXRHMD* OpenXRHMD)
	: FOpenXRActionSet(InInstance, InName, InLocalizedName, InPriority, OpenXRHMD)
{
	Object = InObject;
}

FOpenXRInputPlugin::FOpenXRActionSet::FOpenXRActionSet(XrInstance InInstance,
	const FName& InName, const FString& InLocalizedName, uint32 InPriority, IOpenXRHMD* OpenXRHMD)
	: Handle(XR_NULL_HANDLE)
	, Name(InName)
	, LocalizedName(InLocalizedName)
	, Object()
{
	char ActionName[NAME_SIZE];
	Name.GetPlainANSIString(ActionName);

	XrActionSet ActionSet = XR_NULL_HANDLE;
	XrActionSetCreateInfo Info;
	Info.type = XR_TYPE_ACTION_SET_CREATE_INFO;
	Info.next = nullptr;
	FilterActionName(ActionName, Info.actionSetName);
	if (!InLocalizedName.IsEmpty())
	{
		FTCHARToUTF8_Convert::Convert(Info.localizedActionSetName, XR_MAX_LOCALIZED_ACTION_SET_NAME_SIZE, *InLocalizedName, InLocalizedName.Len() + 1);
	}
	else
	{
		FCStringAnsi::Strncpy(Info.localizedActionSetName, ActionName, XR_MAX_LOCALIZED_ACTION_SET_NAME_SIZE);
	}
	Info.priority = InPriority;

	if (OpenXRHMD)
	{
		for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
		{
			Info.next = Plugin->OnCreateActionSet(Info, Info.next);
		}
	}

	XR_ENSURE(xrCreateActionSet(InInstance, &Info, &Handle));

	if (OpenXRHMD)
	{
		for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
		{
			Plugin->PostCreateActionSet(Handle);
		}
	}

}

FOpenXRInputPlugin::FOpenXRController::FOpenXRController(XrActionSet InActionSet, XrPath InUserPath, const char* InName)
	: ActionSet(InActionSet)
	, UserPath(InUserPath)
	, GripAction(XR_NULL_HANDLE)
	, AimAction(XR_NULL_HANDLE)
	, VibrationAction(XR_NULL_HANDLE)
	, GripDeviceId(-1)
	, AimDeviceId(-1)
	, bHapticActive(false)
{
	XrActionCreateInfo Info;
	Info.type = XR_TYPE_ACTION_CREATE_INFO;
	Info.next = nullptr;
	Info.countSubactionPaths = 0;
	Info.subactionPaths = nullptr;

	FCStringAnsi::Strncpy(Info.localizedActionName, InName, XR_MAX_ACTION_NAME_SIZE);
	FCStringAnsi::StrncatTruncateDest(Info.localizedActionName, XR_MAX_ACTION_NAME_SIZE, " Grip Pose");
	FilterActionName(Info.localizedActionName, Info.actionName);
	Info.actionType = XR_ACTION_TYPE_POSE_INPUT;
	XR_ENSURE(xrCreateAction(ActionSet, &Info, &GripAction));

	FCStringAnsi::Strncpy(Info.localizedActionName, InName, XR_MAX_ACTION_NAME_SIZE);
	FCStringAnsi::StrncatTruncateDest(Info.localizedActionName, XR_MAX_ACTION_NAME_SIZE, " Aim Pose");
	FilterActionName(Info.localizedActionName, Info.actionName);
	Info.actionType = XR_ACTION_TYPE_POSE_INPUT;
	XR_ENSURE(xrCreateAction(ActionSet, &Info, &AimAction));

	FCStringAnsi::Strncpy(Info.localizedActionName, InName, XR_MAX_ACTION_NAME_SIZE);
	FCStringAnsi::StrncatTruncateDest(Info.localizedActionName, XR_MAX_ACTION_NAME_SIZE, " Palm Pose");
	FilterActionName(Info.localizedActionName, Info.actionName);
	Info.actionType = XR_ACTION_TYPE_POSE_INPUT;
	XR_ENSURE(xrCreateAction(ActionSet, &Info, &PalmAction));

	FCStringAnsi::Strncpy(Info.localizedActionName, InName, XR_MAX_ACTION_NAME_SIZE);
	FCStringAnsi::StrncatTruncateDest(Info.localizedActionName, XR_MAX_ACTION_NAME_SIZE, " Vibration");
	FilterActionName(Info.localizedActionName, Info.actionName);
	Info.actionType = XR_ACTION_TYPE_VIBRATION_OUTPUT;
	XR_ENSURE(xrCreateAction(ActionSet, &Info, &VibrationAction));
}

void FOpenXRInputPlugin::FOpenXRController::AddTrackedDevices(IOpenXRHMD* HMD)
{
	if (HMD)
	{
		GripDeviceId = HMD->AddTrackedDevice(GripAction, UserPath);
		AimDeviceId = HMD->AddTrackedDevice(AimAction, UserPath);
		PalmDeviceId = HMD->AddTrackedDevice(PalmAction, UserPath);
	}
}

FOpenXRInputPlugin::FInteractionProfile::FInteractionProfile(XrPath InProfile, bool InHasHaptics)
	: HasHaptics(InHasHaptics)
	, Path(InProfile)
	, Bindings()
{
}

FOpenXRInputPlugin::FOpenXRInput::FOpenXRInput(IXRTrackingSystem* InTrackingSystem)
	: TrackingSystem(InTrackingSystem)
	, OpenXRHMD(InTrackingSystem ? InTrackingSystem->GetIOpenXRHMD() : nullptr)
	, Instance(XR_NULL_HANDLE)
	, ControllerActionSet()
	, ActionSets()
	, SubactionPaths()
	, LegacyActions()
	, EnhancedActions()
	, Controllers()
	, MotionSourceToControllerHandMap()
	, InputMappingContextToPriorityMap()
	, bActionsAttached(false)
	, bDirectionalBindingSupported(false)
	, bPalmPoseSupported(false)
	, bGripSurfacePoseSupported(false)
	, bActionSetPrioritySupported(false)
	, MessageHandler(new FGenericApplicationMessageHandler())
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);

	// If there is no HMD then this module is not active, but it still needs to exist so we can EnumerateMotionSources from it.
	if (OpenXRHMD)
	{
		Instance = OpenXRHMD->GetInstance();
		bDirectionalBindingSupported = OpenXRHMD->IsExtensionEnabled(XR_EXT_DPAD_BINDING_EXTENSION_NAME);
		bPalmPoseSupported = OpenXRHMD->IsExtensionEnabled(XR_EXT_PALM_POSE_EXTENSION_NAME);
		bGripSurfacePoseSupported = OpenXRHMD->IsOpenXRAPIVersionMet(EOpenXRAPIVersion::V_1_1);
		bActionSetPrioritySupported = OpenXRHMD->IsExtensionEnabled(XR_EXT_ACTIVE_ACTION_SET_PRIORITY_EXTENSION_NAME);

		// Note: AnyHand needs special handling because it tries left then falls back to right in each call.
		MotionSourceToControllerHandMap.Add(OpenXRSourceNames::Left, EControllerHand::Left);
		MotionSourceToControllerHandMap.Add(OpenXRSourceNames::Right, EControllerHand::Right);
		MotionSourceToControllerHandMap.Add(OpenXRSourceNames::LeftGrip, EControllerHand::Left);
		MotionSourceToControllerHandMap.Add(OpenXRSourceNames::RightGrip, EControllerHand::Right);
		MotionSourceToControllerHandMap.Add(OpenXRSourceNames::LeftAim, EControllerHand::Left);
		MotionSourceToControllerHandMap.Add(OpenXRSourceNames::RightAim, EControllerHand::Right);
		MotionSourceToControllerHandMap.Add(OpenXRSourceNames::LeftPalm, EControllerHand::Left);
		MotionSourceToControllerHandMap.Add(OpenXRSourceNames::RightPalm, EControllerHand::Right);

		// Map the legacy hand enum values that openxr supports
		MotionSourceToControllerHandMap.Add(TEXT("EControllerHand::Left"), EControllerHand::Left);
		MotionSourceToControllerHandMap.Add(TEXT("EControllerHand::Right"), EControllerHand::Right);
		MotionSourceToControllerHandMap.Add(TEXT("EControllerHand::AnyHand"), EControllerHand::AnyHand);

		// Generate a list of the sub-action paths so we can query the left/right hand individually
		SubactionPaths.Add(FOpenXRPath("/user/hand/left"));
		SubactionPaths.Add(FOpenXRPath("/user/hand/right"));

		OpenXRHMD->SetInputModule(this);
	}
}

XrAction FOpenXRInputPlugin::FOpenXRInput::GetActionForMotionSource(FName MotionSource) const
{
	const EControllerHand* Hand = MotionSourceToControllerHandMap.Find(MotionSource);
	if (Hand == nullptr)
	{
		return XR_NULL_HANDLE;
	}
	const FOpenXRController& Controller = Controllers[*Hand];
	if (MotionSource == OpenXRSourceNames::LeftAim || MotionSource == OpenXRSourceNames::RightAim)
	{
		return Controller.AimAction;
	}
	else if (MotionSource == OpenXRSourceNames::LeftPalm || MotionSource == OpenXRSourceNames::RightPalm)
	{
		return Controller.PalmAction;
	}
	else
	{
		return Controller.GripAction;
	}
}

int32 FOpenXRInputPlugin::FOpenXRInput::GetDeviceIDForMotionSource(FName MotionSource) const
{
	const FOpenXRController& Controller = Controllers[MotionSourceToControllerHandMap.FindChecked(MotionSource)];
	if (MotionSource == OpenXRSourceNames::LeftAim || MotionSource == OpenXRSourceNames::RightAim)
	{
		return Controller.AimDeviceId;
	}
	else if (MotionSource == OpenXRSourceNames::LeftPalm || MotionSource == OpenXRSourceNames::RightPalm)
	{
		return Controller.PalmDeviceId;
	}
	else
	{
		return Controller.GripDeviceId;
	}
}

XrPath FOpenXRInputPlugin::FOpenXRInput::GetUserPathForMotionSource(FName MotionSource) const
{
	const FOpenXRController& Controller = Controllers[MotionSourceToControllerHandMap.FindChecked(MotionSource)];
	return Controller.UserPath;
}

bool FOpenXRInputPlugin::FOpenXRInput::IsOpenXRInputSupportedMotionSource(const FName MotionSource) const
{
	return
		MotionSource == OpenXRSourceNames::AnyHand
		|| MotionSourceToControllerHandMap.Contains(MotionSource);
}

bool FOpenXRInputPlugin::FOpenXRInput::BuildActions(XrSession Session)
{
	if (bActionsAttached)
	{
		return false;
	}

	DestroyActions();

	// Create an engine action set for pose input and haptic output
	ControllerActionSet = MakeUnique<FOpenXRActionSet>(Instance, "controllers", "Controllers", 0);

	XrPath LeftHand = FOpenXRPath("/user/hand/left");
	XrPath RightHand = FOpenXRPath("/user/hand/right");
	XrPath Head = FOpenXRPath("/user/head");

	// Controller poses
	Controllers.Add(EControllerHand::Left, FOpenXRController(ControllerActionSet->Handle, LeftHand, "Left Controller"));
	Controllers.Add(EControllerHand::Right, FOpenXRController(ControllerActionSet->Handle, RightHand, "Right Controller"));
	Controllers.Add(EControllerHand::HMD, FOpenXRController(ControllerActionSet->Handle, Head, "HMD"));

	// Make OpenXRHMD aware of the controller action spaces
	Controllers[EControllerHand::Left].AddTrackedDevices(OpenXRHMD);
	Controllers[EControllerHand::Right].AddTrackedDevices(OpenXRHMD);

	// Generate a map of all supported interaction profiles to store suggested bindings
	TMap<FString, FInteractionProfile> Profiles;
	{
		auto AddProfile = [&Profiles](const char* const Profile, const char* const PathChars, bool HasHaptics)
			{
				FOpenXRPath Path = FOpenXRPath(PathChars);
				if (XR_NULL_PATH == Path.ToXRPath())
				{
					UE_LOGF(LogHMD, Warning, "BuildActions tried to AddProfile: '%ls' Path: '%ls', but it looks like the path creation failed. Profile not added. Look above for path related error messages.", ANSI_TO_TCHAR(Profile), ANSI_TO_TCHAR(PathChars));
					return;
				}
				Profiles.Add(Profile, FInteractionProfile(Path, HasHaptics));
			};
		//AddProfile("MalformedPathTest", "MALFORMEDPATH *!,/", false); // malformed path, should fail with XR_ERROR_PATH_FORMAT_INVALID = -21
		AddProfile("SimpleController", "/interaction_profiles/khr/simple_controller", true);
		AddProfile("Vive", "/interaction_profiles/htc/vive_controller", true);
		AddProfile("MixedReality", "/interaction_profiles/microsoft/motion_controller", true);
		AddProfile("OculusGo", "/interaction_profiles/oculus/go_controller", false);
		AddProfile("OculusTouch", "/interaction_profiles/oculus/touch_controller", true);
		AddProfile("ValveIndex", "/interaction_profiles/valve/index_controller", true);
	}

	// Query extension plugins for input key overrides
	for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
	{
		TArray<FInputKeyOpenXRProperties> PluginInputOverrides;
		if (Plugin->GetInputKeyOverrides(PluginInputOverrides))
		{
			for (const FInputKeyOpenXRProperties& PluginInputOverride : PluginInputOverrides)
			{
				InputsKeysToPropertiesMap.FindOrAdd(PluginInputOverride.InputKey).Add(PluginInputOverride);
			}
		}
	}

	// Query extension plugins for interaction profiles
	{
		TArray<FString> KeyPrefixes;
		TArray<XrPath> Paths;
		TArray<bool> Haptics;
		for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
		{
			FString KeyPrefix;
			XrPath Path = XR_NULL_PATH;
			bool HasHaptics = false;
	
			KeyPrefixes.Reset();
			Paths.Reset();
			Haptics.Reset();
			if (Plugin->GetInteractionProfiles(Instance, KeyPrefixes, Paths, Haptics) && Paths.Num() != 0)
			{
				for (int i = 0; i < Paths.Num(); i++)
				{
					if (Paths[i] != XR_NULL_PATH)
					{
						Profiles.Add(KeyPrefixes[i], FInteractionProfile(Paths[i], Haptics[i]));
					}
				}
			}
		}
	}
	
	// Attempt to load the default input config from the OpenXR input settings.
	const UEnhancedInputDeveloperSettings* InputSettings = GetDefault<UEnhancedInputDeveloperSettings>();
	if (InputSettings && InputSettings->bEnableDefaultMappingContexts)
	{
		for (const auto& Context : InputSettings->DefaultMappingContexts)
		{
			if (Context.InputMappingContext)
			{
				TStrongObjectPtr<const UInputMappingContext> Obj(Context.InputMappingContext.LoadSynchronous());
				InputMappingContextToPriorityMap.Add(Obj, Context.Priority);
			}
			else
			{
				UE_LOGF(LogHMD, Warning, "Default Mapping Contexts contains an Input Mapping Context set to \"None\", ignoring while building OpenXR actions.");
			}
		}
	}

	if (!InputMappingContextToPriorityMap.IsEmpty())
	{
		BuildEnhancedActions(Profiles);
	}
	else
	{
		UE_LOGF(LogHMD, Warning, "No mapping context provided in the OpenXR Input project settings, action bindings will not be visible to the OpenXR runtime. Action/Axis mappings from the Input configuration are deprecated for XR in favor of Enhanced Input.");

		BuildLegacyActions(Profiles);
	}

	for (TPair<FString, FInteractionProfile>& Pair : Profiles)
	{
		FInteractionProfile& Profile = Pair.Value;

		// Only suggest interaction profile bindings if the developer has provided bindings for them
		// An exception is made for the Simple Controller Profile which is always bound as a fallback
		if (Profile.Bindings.Num() > 0)
		{
			// Add bindings from the extension plugins
			for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
			{
				TArray<XrActionSuggestedBinding> PluginBindings;
				if (Plugin->GetSuggestedBindings(Profile.Path, PluginBindings))
				{
					Profile.Bindings.Append(PluginBindings);
				}
			}

			// Add the bindings for the controller pose and haptics
			Profile.Bindings.Add(XrActionSuggestedBinding{
				Controllers[EControllerHand::Left].GripAction, FOpenXRPath("/user/hand/left/input/grip/pose")
				});
			Profile.Bindings.Add(XrActionSuggestedBinding{
				Controllers[EControllerHand::Right].GripAction, FOpenXRPath("/user/hand/right/input/grip/pose")
				});
			Profile.Bindings.Add(XrActionSuggestedBinding{
				Controllers[EControllerHand::Left].AimAction, FOpenXRPath("/user/hand/left/input/aim/pose")
				});
			Profile.Bindings.Add(XrActionSuggestedBinding{
				Controllers[EControllerHand::Right].AimAction, FOpenXRPath("/user/hand/right/input/aim/pose")
				});

			// Favor the openxr 1.1 core spec version, then the 1.0 extension, then fall back to the grip just so we have something.
			if (bGripSurfacePoseSupported)
			{
				Profile.Bindings.Add(XrActionSuggestedBinding{
					Controllers[EControllerHand::Left].PalmAction, FOpenXRPath("/user/hand/left/input/grip_surface/pose")
					});
				Profile.Bindings.Add(XrActionSuggestedBinding{
					Controllers[EControllerHand::Right].PalmAction, FOpenXRPath("/user/hand/right/input/grip_surface/pose")
					});
			}
			else if (bPalmPoseSupported)
			{
				Profile.Bindings.Add(XrActionSuggestedBinding{
					Controllers[EControllerHand::Left].PalmAction, FOpenXRPath("/user/hand/left/input/palm_ext/pose")
					});
				Profile.Bindings.Add(XrActionSuggestedBinding{
					Controllers[EControllerHand::Right].PalmAction, FOpenXRPath("/user/hand/right/input/palm_ext/pose")
					});
			}
			else
			{
				Profile.Bindings.Add(XrActionSuggestedBinding{
					Controllers[EControllerHand::Left].PalmAction, FOpenXRPath("/user/hand/left/input/grip/pose")
					});
				Profile.Bindings.Add(XrActionSuggestedBinding{
					Controllers[EControllerHand::Right].PalmAction, FOpenXRPath("/user/hand/right/input/grip/pose")
					});
			}

			if (Profile.HasHaptics)
			{
				Profile.Bindings.Add(XrActionSuggestedBinding{
					Controllers[EControllerHand::Left].VibrationAction, FOpenXRPath("/user/hand/left/output/haptic")
					});
				Profile.Bindings.Add(XrActionSuggestedBinding{
					Controllers[EControllerHand::Right].VibrationAction, FOpenXRPath("/user/hand/right/output/haptic")
					});
			}

			XrInteractionProfileSuggestedBinding InteractionProfile;
			InteractionProfile.type = XR_TYPE_INTERACTION_PROFILE_SUGGESTED_BINDING;
			InteractionProfile.next = nullptr;
			InteractionProfile.interactionProfile = Profile.Path;
			InteractionProfile.countSuggestedBindings = Profile.Bindings.Num();
			InteractionProfile.suggestedBindings = Profile.Bindings.GetData();

			for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
			{
				InteractionProfile.next = Plugin->OnSuggestBindings(Profile.Path, InteractionProfile.next);
			}

			XR_ENSURE(xrSuggestInteractionProfileBindings(Instance, &InteractionProfile));
		}
	}

	// Bind the project action sets
	TSet<XrActionSet> AttachSet;
	for (auto&& ActionSet : ActionSets)
		AttachSet.Add(ActionSet.Handle);

	// Bind the controller action set
	if (ControllerActionSet)
	{
		AttachSet.Add(ControllerActionSet->Handle);
	}

	// Bind plugin action sets exposed through a query
	for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
	{
		TSet<XrActionSet> PluginAttachSet;
		Plugin->AttachActionSets(PluginAttachSet);
		AttachSet.Append(PluginAttachSet);
	}

	TArray<XrActionSet> AttachArray = AttachSet.Array();
	XrSessionActionSetsAttachInfo SessionActionSetsAttachInfo;
	SessionActionSetsAttachInfo.type = XR_TYPE_SESSION_ACTION_SETS_ATTACH_INFO;
	SessionActionSetsAttachInfo.next = nullptr;
	SessionActionSetsAttachInfo.countActionSets = AttachArray.Num();
	SessionActionSetsAttachInfo.actionSets = AttachArray.GetData();

	for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
	{
		SessionActionSetsAttachInfo.next = Plugin->OnActionSetAttach(SessionActionSetsAttachInfo, SessionActionSetsAttachInfo.next);
	}

	bActionsAttached = XR_ENSURE(xrAttachSessionActionSets(Session, &SessionActionSetsAttachInfo));
	// NOTE: xrAttachSessionActionSets may fail with XR_ERROR_HANDLE_INVALID when using the Meta openxr runtime because the VivePort eye tracking openxr layer is incompatible with that runtime.
	// Not sure what to do about that at the moment.  Perhaps we could simply log a warning when the layer is seen indicating that uninstalling it might fix problems?  
	// But to help consumers that would need to pop up a UI or something.

	return bActionsAttached;
}

void FOpenXRInputPlugin::FOpenXRInput::BuildLegacyActions(TMap<FString, FInteractionProfile>& Profiles)
{
	// Create an engine action set for legacy actions
	FOpenXRActionSet ActionSet(Instance, "ue", "Unreal Engine", 0);

	TArray<FKey> Keys;
	EKeys::GetAllKeys(Keys);
	for (const FKey& Key : Keys)
	{
		// XR Keys are never blueprint-bindable
		// 2D Axis keys are supported through paired keys
		if (Key.IsDeprecated() || Key.IsBindableInBlueprints() || (Key.IsAnalog() && !Key.IsAxis1D()))
		{
			continue;
		}

		XrActionType Type = Key.IsAxis1D() ? XR_ACTION_TYPE_FLOAT_INPUT : XR_ACTION_TYPE_BOOLEAN_INPUT;
		FOpenXRAction Action(ActionSet.Handle, Type, Key.GetFName(), Key.GetDisplayName().ToString(), SubactionPaths);
		if (SuggestBindingForKey(Profiles, Action, Key))
		{
			LegacyActions.Add(Action);
		}
		else
		{
			XR_ENSURE(xrDestroyAction(Action.Handle));
		}
	}

	ActionSets.Emplace(MoveTemp(ActionSet));
}

void FOpenXRInputPlugin::FOpenXRInput::BuildEnhancedActions(TMap<FString, FInteractionProfile>& Profiles)
{
	if (bActionsAttached)
	{
		// Don't rebuild actions while another input config has already been attached
		return;
	}

	// OpenXR does not allow duplicated localized names for mapping contexts.  In order to have good warnings and allow input to function even with bad localization setup we will detect duplicates ourselves.
	TMap<FString, FName> MappingContextDescriptionMap;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (const auto& MappingContext : InputMappingContextToPriorityMap)
	{
		FName NewMappingContextName = MappingContext.Key->GetFName();
		FString NewMappingContextDescription = MappingContext.Key->ContextDescription.ToString();

		// Handle any description string duplicates
		{
			FName* ExistingMappingContextName = MappingContextDescriptionMap.Find(NewMappingContextDescription);
			if (ExistingMappingContextName != nullptr)
			{
				UE_LOGF(LogHMD, Warning, "Input Mapping Context %ls has a Description, \"%ls\", which exactly matches the Description already used by Input Mapping Context %ls.  Identical localized descriptions are not allowed by OpenXR.  The FName of this mapping context %ls, which is unique, will replace the duplicated localized string so the Input Mapping Context functions but it will not localize correctly."
					, *NewMappingContextName.ToString(), *NewMappingContextDescription, *ExistingMappingContextName->ToString(), *NewMappingContextName.ToString());
				NewMappingContextDescription = NewMappingContextName.ToString();
			}
			else
			{
				MappingContextDescriptionMap.Add(NewMappingContextDescription, NewMappingContextName);
			}
		}

		FOpenXRActionSet ActionSet(Instance, NewMappingContextName, NewMappingContextDescription, MappingContext.Value, MappingContext.Key.Get());


		// OpenXR does not allow duplicated localized names for actions.  In order to have good warnings allow input to function even with bad localization setup we will detect duplicates ourselves.
		TMap<FString, FName> ActionDescriptionMap;

		TMap<FName, int32> ActionMap;

		for (const FEnhancedActionKeyMapping& Mapping : MappingContext.Key->GetMappings())
		{
			if (!Mapping.Action)
			{
				continue;
			}

			// Try to find an existing action within the current action set
			FName NewActionName = Mapping.Action->GetFName();
			int32& ActionIndex = ActionMap.FindOrAdd(NewActionName, INDEX_NONE);
			if (ActionIndex == INDEX_NONE)
			{
				// No action found, create a new one

				if (!ensureMsgf(!NewActionName.GetDisplayNameEntry()->IsWide(), TEXT("Input Action %s has wide characters in its name, This name will be passed to OpenXR which does not support wide characters in path names (see spec).  Ignoring this action, it will not function."), *NewActionName.ToString()))
				{
					continue;
				}

				FString NewActionDescription = Mapping.Action->ActionDescription.ToString();
				{
					// Handle any description string duplicates
					FName* ExistingActionName = ActionDescriptionMap.Find(NewActionDescription);
					if (ExistingActionName != nullptr)
					{
						UE_LOGF(LogHMD, Warning, "Input Action %ls has a Description, \"%ls\", which exactly matches the Description already used by Action %ls.  Identical localized descriptions are not allowed by OpenXR.  The FName of this action %ls, which is unique, will replace the localized string so that the input Action functions but it will not localize correctly."
							, *NewActionName.ToString(), *NewActionDescription, *ExistingActionName->ToString(), *NewActionName.ToString());
						NewActionDescription = NewActionName.ToString();
					}
					else
					{
						ActionDescriptionMap.Add(NewActionDescription, NewActionName);
					}
				}

				XrActionType ActionType = ToActionType(Mapping.Action->ValueType);
				if (!ActionType)
				{
					continue;
				}

				// Create the action and write the index to the reference in the actions map
				ActionIndex = EnhancedActions.Emplace(ActionSet.Handle, ActionType, NewActionName, NewActionDescription, SubactionPaths, Mapping.Action);
			}

			SuggestBindingForKey(Profiles, EnhancedActions[ActionIndex], Mapping.Key, Mapping.Modifiers, Mapping.Triggers);
		}
		ActionSets.Emplace(MoveTemp(ActionSet));
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FOpenXRInputPlugin::FOpenXRInput::DestroyActions()
{
	// Destroying an action set will also destroy all actions in the set
	for (const FOpenXRActionSet& ActionSet : ActionSets)
	{
		xrDestroyActionSet(ActionSet.Handle);
	}

	if (ControllerActionSet)
	{
		xrDestroyActionSet(ControllerActionSet->Handle);
		ControllerActionSet.Reset();
	}

	LegacyActions.Reset();
	EnhancedActions.Reset();
	ActionSets.Reset();
	Controllers.Reset();
}

template<typename T>
int32 FOpenXRInputPlugin::FOpenXRInput::SuggestBindings(TMap<FString, FInteractionProfile>& Profiles, FOpenXRAction& Action, const TArray<T>& Mappings)
{
	int32 SuggestedBindings = 0;

	// Add suggested bindings for every mapping
	for (const T& InputKey : Mappings)
	{
		if (SuggestBindingForKey(Profiles, Action, InputKey.Key))
		{
			++SuggestedBindings;
		}
	}

	return SuggestedBindings;
}

bool FOpenXRInputPlugin::FOpenXRInput::SuggestBindingForKey(TMap<FString, FInteractionProfile>& Profiles, FOpenXRAction& Action, const FKey& InFKey, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers)
{
	// Use profiles and path from overrides, if an entry exists for the key
	FString InputKey = InFKey.ToString();
	if (InputsKeysToPropertiesMap.Contains(InputKey))
	{
		for (const FInputKeyOpenXRProperties& InputProperties : InputsKeysToPropertiesMap[InputKey])
		{
			FInteractionProfile* Profile = Profiles.Find(InputProperties.InteractionProfile);
			if (Profile == nullptr)
			{
				continue;
			}
			Profile->Bindings.Add(XrActionSuggestedBinding{ Action.Handle, FOpenXRPath(InputProperties.OpenXRPath) });
		}
		return true;
	}

	// Key names that are parseable into an OpenXR path have exactly 4 tokens
	TArray<FString> Tokens;
	if (InFKey.ToString().ParseIntoArray(Tokens, TEXT("_")) != EKeys::NUM_XR_KEY_TOKENS)
	{
		return false;
	}

	// Check if we support the profile specified in the key name
	FInteractionProfile* Profile = Profiles.Find(Tokens[0]);
	if (!Profile)
	{
		return false;
	}

	// Parse the key name into an OpenXR interaction profile path
	FString Path = "/user/hand/" + Tokens[1].ToLower();
	XrPath TopLevel = FOpenXRPath(Path);

	// Map this key to the correct subaction for this profile
	// We'll use this later to retrieve binding modifiers
	TPair<XrPath, XrPath> Key(Profile->Path, TopLevel);
	for (UInputTrigger* Trigger : Triggers)
	{
		TObjectPtr<UInputTrigger> Ptr = Trigger;
		Action.Triggers.AddUnique(Key, Ptr);
	}
	for (UInputModifier* Modifier : Modifiers)
	{
		TObjectPtr<UInputModifier> Ptr = Modifier;
		Action.Modifiers.AddUnique(Key, Ptr);
	}

	// Add the input we want to query with grip being defined as "squeeze" in OpenXR
	FString Identifier = Tokens[2].ToLower();
	if (Identifier == "grip")
	{
		Identifier = "squeeze";
	}
	else if(Identifier.StartsWith(TEXT("dpad")))
	{
		if(bDirectionalBindingSupported)
		{
			// dpad inputs are specified as <ProfileName>_<Hand>_DPad<Direction>_<action>.  Normally, we'd have a _ between
			// "dpad" and the direction of the button, but in order to match the hard NUM_XR_KEY_TOKENS
			// formatting requirement, we detect if the third Token starts with "dpad," and extract the direction in order
			// to form the longer path required.
			Identifier = "dpad_" + Identifier.Mid(4);
		}
		else
		{
			return false;
		}
	}
	Path += "/input/" + Identifier;

	// Add the data we want to query, we'll skip this for trigger/squeeze "click" actions to allow
	// certain profiles that don't have "click" data to threshold the "value" data instead
	FString Component = Tokens[3].ToLower();
	if (Component == "axis")
	{
		Path += "/value";
	}
	else if (Component == "click")
	{
		if (Tokens[0] == "ValveIndex" && (Identifier == "trackpad" || Identifier == "squeeze"))
		{
			return false;
		}

		if (Identifier != "trigger" && Identifier != "squeeze")
		{
			Path += "/click";
		}
	}
	else if (Component == "touch")
	{
		Path += "/touch";
	}
	else if (Component == "touchaxis")
	{
		Path += "/touchvalue";  // Note: this is not a standard openxr identifier.  It is meant to represent some kind of analog touch sensor.
	}
	else if (Component == "up" || Component == "down" || Component == "left" || Component == "right")
	{
		if (!bDirectionalBindingSupported)
		{
			return false;
		}
		Path += "/dpad_" + Component;
	}
	else if (Component != "2d")
	{
		// Anything we don't need to translate can pass through
		// Except for 2D vectors, which don't need a component path
		Path += "/" + Component;
	}

	// Add the binding to the profile
	Profile->Bindings.Add(XrActionSuggestedBinding{ Action.Handle, FOpenXRPath(Path) });
	return true;
}

bool FOpenXRInputPlugin::FOpenXRInput::SuggestBindingForKey(TMap<FString, FInteractionProfile>& Profiles, FOpenXRAction& Action, const FKey& InFKey)
{
	TArray<UInputModifier*> Modifiers;
	TArray<UInputTrigger*> Triggers;
	return SuggestBindingForKey(Profiles, Action, InFKey, Modifiers, Triggers);
}

void FOpenXRInputPlugin::FOpenXRInput::OnBeginSession()
{
	check(OpenXRHMD); // If there is no hmd this function should not be called.

	XrSession Session = OpenXRHMD->GetSession();
	if (Session != XR_NULL_HANDLE)
	{
		if (!bActionsAttached)
		{
			BuildActions(Session);
		}
	}
}

void FOpenXRInputPlugin::FOpenXRInput::OnDestroySession()
{
	if (bActionsAttached)
	{
		// If the session shut down, clean up.
		bActionsAttached = false;
		InputMappingContextToPriorityMap.Reset();
	}
}

void FOpenXRInputPlugin::FOpenXRInput::SyncActions(XrSession Session)
{
	if (OpenXRHMD->IsFocused() && bActionsAttached)
	{
		TMap<XrActionSet, int32> ActiveSet;
		auto GetActiveForSubsystem = [this, &ActiveSet](IEnhancedInputSubsystemInterface* Subsystem)
			{
				if (Subsystem)
				{
					for (const FOpenXRActionSet& ActionSet : ActionSets)
					{
						int32 Priority = 0;
						if (ActionSet.Object && Subsystem->HasMappingContext(ActionSet.Object, Priority))
						{
							int32* PriorityPtr = ActiveSet.Find(ActionSet.Handle);
							if (PriorityPtr)
							{
								*PriorityPtr = FMath::Max(*PriorityPtr, Priority);
							}
							else
							{
								ActiveSet.Add(ActionSet.Handle, Priority);
							}
						}
					}
				}
			};

		IEnhancedInputModule::Get().GetLibrary()->ForEachSubsystem(GetActiveForSubsystem);

#if WITH_EDITOR
		if (GEditor)
		{
			// UEnhancedInputLibrary::ForEachSubsystem only enumerates runtime subsystems.
			GetActiveForSubsystem(GEditor->GetEditorSubsystem<UEnhancedInputEditorSubsystem>());
		}
#endif

		TArray<XrActiveActionSet> ActiveActionSets;
		TArray<XrActiveActionSetPriorityEXT> ActivePriorities;
		for (const auto& ActionSet : ActiveSet)
		{
			ActiveActionSets.Add(XrActiveActionSet{ ActionSet.Key, XR_NULL_PATH });
			ActivePriorities.Add(XrActiveActionSetPriorityEXT{ ActionSet.Key, ToXrPriority(ActionSet.Value) });
		}

		// If legacy actions are enabled, all action sets are always active
		if (!LegacyActions.IsEmpty())
		{
			for (const FOpenXRActionSet& ActionSet : ActionSets)
			{
				ActiveActionSets.Add(XrActiveActionSet{ ActionSet.Handle, XR_NULL_PATH });
			}
		}

		// The controller is always active
		if (ControllerActionSet)
		{
			ActiveActionSets.Add(XrActiveActionSet{ ControllerActionSet->Handle, XR_NULL_PATH });
		}

		XrActionsSyncInfo SyncInfo = { XR_TYPE_ACTIONS_SYNC_INFO };
		XrActiveActionSetPrioritiesEXT PrioritiesInfo = { XR_TYPE_ACTIVE_ACTION_SET_PRIORITIES_EXT };
		// This is a workaround to avoid log spam from some OpenXR runtimes when they do not support this extension.
		if (bActionSetPrioritySupported)
		{
			SyncInfo.next = &PrioritiesInfo;
			PrioritiesInfo.actionSetPriorityCount = ActivePriorities.Num();
			PrioritiesInfo.actionSetPriorities = ActivePriorities.GetData();
		}

		for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
		{
			Plugin->GetActiveActionSetsForSync(ActiveActionSets);

			SyncInfo.next = Plugin->OnSyncActions(Session, SyncInfo.next);
		}

		SyncInfo.countActiveActionSets = ActiveActionSets.Num();
		SyncInfo.activeActionSets = ActiveActionSets.GetData();

		XR_ENSURE(xrSyncActions(Session, &SyncInfo));

		for (IOpenXRExtensionPlugin* Plugin : OpenXRHMD->GetExtensionPlugins())
		{
			Plugin->PostSyncActions(Session);
		}
	}
}

namespace OpenXRInputNamespace
{
	FXRTimedInputActionDelegate* GetTimedInputActionDelegate(FName ActionName)
	{
		FXRTimedInputActionDelegate* XRTimedInputActionDelegate = UHeadMountedDisplayFunctionLibrary::OnXRTimedInputActionDelegateMap.Find(ActionName);
		if (XRTimedInputActionDelegate && !XRTimedInputActionDelegate->IsBound())
		{
			XRTimedInputActionDelegate = nullptr;
		}
		return XRTimedInputActionDelegate;
	}
}

void FOpenXRInputPlugin::FOpenXRInput::SendControllerEvents()
{
	if (!bActionsAttached || OpenXRHMD == nullptr)
	{
		return;
	}

	if (!OpenXRHMD->IsFocused())
	{
		return;
	}

	XrSession Session = OpenXRHMD->GetSession();

	if (Session != XR_NULL_HANDLE)
	{
		SyncActions(Session);
	}

	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
	for (FOpenXRAction& Action : LegacyActions)
	{
		XrActionStateGetInfo GetInfo;
		GetInfo.type = XR_TYPE_ACTION_STATE_GET_INFO;
		GetInfo.next = nullptr;
		GetInfo.subactionPath = XR_NULL_PATH;
		GetInfo.action = Action.Handle;

		switch (Action.Type)
		{
			case XR_ACTION_TYPE_BOOLEAN_INPUT:
			{
				XrActionStateBoolean State;
				State.type = XR_TYPE_ACTION_STATE_BOOLEAN;
				State.next = nullptr;
				XrResult Result = xrGetActionStateBoolean(Session, &GetInfo, &State);
				if (XR_SUCCEEDED(Result) && State.changedSinceLastSync)
				{
					if (State.isActive && State.currentState)
					{
						// TODO: Map input devices here if OpenXR would like to
						MessageHandler->OnControllerButtonPressed(Action.Name, DeviceMapper.GetPrimaryPlatformUser(), DeviceMapper.GetDefaultInputDevice(), /*IsRepeat =*/false);
					}
					else
					{
						// TODO: Map input devices here if OpenXR would like to
						MessageHandler->OnControllerButtonReleased(Action.Name, DeviceMapper.GetPrimaryPlatformUser(), DeviceMapper.GetDefaultInputDevice(), /*IsRepeat =*/false);
					}

					Action.NextRepeatTime = State.lastChangeTime + InitialButtonRepeatDelay;

					FXRTimedInputActionDelegate* const Delegate = OpenXRInputNamespace::GetTimedInputActionDelegate(Action.Name);
					if (Delegate)
					{
						Delegate->Execute(State.currentState ? 1.0 : 0.0f, ToFTimespan(State.lastChangeTime));
					}
				}
				else if (State.currentState && OpenXRHMD->GetDisplayTime() >= Action.NextRepeatTime)
				{
					// TODO: We should retrieve the current time rather than the display time
					MessageHandler->OnControllerButtonPressed(Action.Name, DeviceMapper.GetPrimaryPlatformUser(), DeviceMapper.GetDefaultInputDevice(), /*IsRepeat =*/true);
					Action.NextRepeatTime = OpenXRHMD->GetDisplayTime() + ButtonRepeatDelay;
				}
			}
			break;
			case XR_ACTION_TYPE_FLOAT_INPUT:
			{
				XrActionStateFloat State;
				State.type = XR_TYPE_ACTION_STATE_FLOAT;
				State.next = nullptr;
				XrResult Result = xrGetActionStateFloat(Session, &GetInfo, &State);
				if (XR_SUCCEEDED(Result) && State.changedSinceLastSync)
				{
					if (State.isActive)
					{
						// TODO: Map input devices here if OpenXR would like to
						MessageHandler->OnControllerAnalog(Action.Name, DeviceMapper.GetPrimaryPlatformUser(), DeviceMapper.GetDefaultInputDevice(), State.currentState);
					}
					else
					{
						// TODO: Map input devices here if OpenXR would like to
						MessageHandler->OnControllerAnalog(Action.Name, DeviceMapper.GetPrimaryPlatformUser(), DeviceMapper.GetDefaultInputDevice(), 0.0f);
					}

					FXRTimedInputActionDelegate* const Delegate = OpenXRInputNamespace::GetTimedInputActionDelegate(Action.Name);
					if (Delegate)
					{
						Delegate->Execute(State.currentState, ToFTimespan(State.lastChangeTime));
					}
				}
			}
			break;
			default:
			// Other action types are currently unsupported.
			break;
		}
	}

	for (XrPath Subaction : SubactionPaths)
	{
		XrInteractionProfileState Profile;
		Profile.type = XR_TYPE_INTERACTION_PROFILE_STATE;
		Profile.next = nullptr;
		XR_ENSURE(xrGetCurrentInteractionProfile(Session, Subaction, &Profile));

		TPair<XrPath, XrPath> Key(Profile.interactionProfile, Subaction);
		for (FOpenXRAction& Action : EnhancedActions)
		{
			const UInputAction* InputAction = Action.Object;

			XrActionStateGetInfo GetInfo;
			GetInfo.type = XR_TYPE_ACTION_STATE_GET_INFO;
			GetInfo.next = nullptr;
			GetInfo.subactionPath = Subaction;
			GetInfo.action = Action.Handle;

			FInputActionValue InputValue;
			switch (Action.Type)
			{
			case XR_ACTION_TYPE_BOOLEAN_INPUT:
			{
				XrActionStateBoolean State;
				State.type = XR_TYPE_ACTION_STATE_BOOLEAN;
				State.next = nullptr;
				XrResult Result = xrGetActionStateBoolean(Session, &GetInfo, &State);
				if (XR_SUCCEEDED(Result))
				{
					InputValue = FInputActionValue(State.isActive ? (bool)State.currentState : false);
				}
				else
				{
					continue;
				}
			}
			break;
			case XR_ACTION_TYPE_FLOAT_INPUT:
			{
				XrActionStateFloat State;
				State.type = XR_TYPE_ACTION_STATE_FLOAT;
				State.next = nullptr;
				XrResult Result = xrGetActionStateFloat(Session, &GetInfo, &State);
				if (XR_SUCCEEDED(Result))
				{
					InputValue = FInputActionValue(State.isActive ? State.currentState : 0.0f);
				}
				else
				{
					continue;
				}
			}
			break;
			case XR_ACTION_TYPE_VECTOR2F_INPUT:
			{
				XrActionStateVector2f State;
				State.type = XR_TYPE_ACTION_STATE_VECTOR2F;
				State.next = nullptr;
				XrResult Result = xrGetActionStateVector2f(Session, &GetInfo, &State);
				if (XR_SUCCEEDED(Result))
				{
					InputValue = FInputActionValue(State.isActive ? ToFVector2D(State.currentState) : FVector2D::ZeroVector);
				}
				else
				{
					continue;
				}
			}
			break;
			default:
				// Other action types are currently unsupported.
				continue;
			}

			TArray<TObjectPtr<UInputTrigger>> Triggers;
			Action.Triggers.MultiFind(Key, Triggers, false);
			TArray<TObjectPtr<UInputModifier>> Modifiers;
			Action.Modifiers.MultiFind(Key, Modifiers, false);

			auto InjectSubsystemInput = [InputAction, InputValue, Triggers, Modifiers](IEnhancedInputSubsystemInterface* Subsystem)
				{
					if (Subsystem)
					{
						Subsystem->InjectInputForAction(InputAction, InputValue, Modifiers, Triggers);
					}
				};

			IEnhancedInputModule::Get().GetLibrary()->ForEachSubsystem(InjectSubsystemInput);

#if WITH_EDITOR
			if (GEditor)
			{
				// UEnhancedInputLibrary::ForEachSubsystem only enumerates runtime subsystems.
				InjectSubsystemInput(GEditor->GetEditorSubsystem<UEnhancedInputEditorSubsystem>());
			}
#endif
		}
	}
}

void FOpenXRInputPlugin::FOpenXRInput::SetMessageHandler(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler)
{
	MessageHandler = InMessageHandler;
}

bool FOpenXRInputPlugin::FOpenXRInput::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	return false;
}

void FOpenXRInputPlugin::FOpenXRInput::SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
	// Large channel type maps to amplitude. We are interested in amplitude.
	if ((ChannelType == FForceFeedbackChannelType::LEFT_LARGE) ||
		(ChannelType == FForceFeedbackChannelType::RIGHT_LARGE))
	{
		FHapticFeedbackValues Values(XR_FREQUENCY_UNSPECIFIED, Value);
		SetHapticFeedbackValues(ControllerId, ChannelType == FForceFeedbackChannelType::LEFT_LARGE ? (int32)EControllerHand::Left : (int32)EControllerHand::Right, Values);
	}
}

void FOpenXRInputPlugin::FOpenXRInput::SetChannelValues(int32 ControllerId, const FForceFeedbackValues &values)
{
	FHapticFeedbackValues leftHaptics = FHapticFeedbackValues(
		values.LeftSmall,		// frequency
		values.LeftLarge);		// amplitude
	FHapticFeedbackValues rightHaptics = FHapticFeedbackValues(
		values.RightSmall,		// frequency
		values.RightLarge);		// amplitude

	SetHapticFeedbackValues(
		ControllerId,
		(int32)EControllerHand::Left,
		leftHaptics);

	SetHapticFeedbackValues(
		ControllerId,
		(int32)EControllerHand::Right,
		rightHaptics);
}

bool FOpenXRInputPlugin::FOpenXRInput::SupportsForceFeedback(int32 ControllerId)
{
	if (!bActionsAttached || OpenXRHMD == nullptr)
	{
		return false;
	}

	XrSession Session = OpenXRHMD->GetSession();
	return Session != XR_NULL_HANDLE;
}

void FOpenXRInputPlugin::FOpenXRInput::SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property)
{
	if (OpenXRHMD == nullptr)
	{
		return;
	}

	for (IOpenXRExtensionPlugin* Module : OpenXRHMD->GetExtensionPlugins())
	{
		Module->OnSetDeviceProperty(OpenXRHMD->GetSession(), ControllerId, Property);
	}
}

FName FOpenXRInputPlugin::FOpenXRInput::GetMotionControllerDeviceTypeName() const
{
	return FName(TEXT("OpenXR"));
}


bool FOpenXRInputPlugin::FOpenXRInput::GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const
{
	if (!bActionsAttached || OpenXRHMD == nullptr)
	{
		return false;
	}

	if (ControllerIndex == 0 && IsOpenXRInputSupportedMotionSource(MotionSource))
	{
		if (MotionSource == OpenXRSourceNames::AnyHand)
		{
			return GetControllerOrientationAndPosition(ControllerIndex, OpenXRSourceNames::LeftGrip, OutOrientation, OutPosition, WorldToMetersScale)
				|| GetControllerOrientationAndPosition(ControllerIndex, OpenXRSourceNames::RightGrip, OutOrientation, OutPosition, WorldToMetersScale);
		}

		if (MotionSource == OpenXRSourceNames::Left)
		{
			return GetControllerOrientationAndPosition(ControllerIndex, OpenXRSourceNames::LeftGrip, OutOrientation, OutPosition, WorldToMetersScale);
		}

		if (MotionSource == OpenXRSourceNames::Right)
		{
			return GetControllerOrientationAndPosition(ControllerIndex, OpenXRSourceNames::RightGrip, OutOrientation, OutPosition, WorldToMetersScale);
		}

		XrSession Session = OpenXRHMD->GetSession();

		if (Session == XR_NULL_HANDLE)
		{
			return false;
		}

		XrActionStateGetInfo GetInfo;
		GetInfo.type = XR_TYPE_ACTION_STATE_GET_INFO;
		GetInfo.next = nullptr;
		GetInfo.subactionPath = XR_NULL_PATH;
		GetInfo.action = GetActionForMotionSource(MotionSource);

		XrActionStatePose State;
		State.type = XR_TYPE_ACTION_STATE_POSE;
		State.next = nullptr;
		XrResult Result = xrGetActionStatePose(Session, &GetInfo, &State);
		if (Result >= XR_SUCCESS && State.isActive)
		{
			FQuat Orientation;
			TrackingSystem->GetCurrentPose(GetDeviceIDForMotionSource(MotionSource), Orientation, OutPosition);
			OutOrientation = FRotator(Orientation);
			return true;
		}
	}

	return false;
}

bool FOpenXRInputPlugin::FOpenXRInput::GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const
{
	// FTimespan initializes to 0 and GetControllerOrientationAndPositionForTime with time 0 will return the latest data.
	FTimespan Time;
	bool OutTimeWasUsed = false;
	return GetControllerOrientationAndPositionForTime(ControllerIndex, MotionSource, Time, OutTimeWasUsed, OutOrientation, OutPosition, OutbProvidedLinearVelocity, OutLinearVelocity, OutbProvidedAngularVelocity, OutAngularVelocityAsAxisAndLength, OutbProvidedLinearAcceleration, OutLinearAcceleration, WorldToMetersScale);
}

bool FOpenXRInputPlugin::FOpenXRInput::GetControllerOrientationAndPositionForTime(const int32 ControllerIndex, const FName MotionSource, FTimespan Time, bool& OutTimeWasUsed, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const
{
	if (!bActionsAttached || OpenXRHMD == nullptr)
	{
		return false;
	}

	if (ControllerIndex == 0 && IsOpenXRInputSupportedMotionSource(MotionSource))
	{
		if (MotionSource == OpenXRSourceNames::AnyHand)
		{
			return GetControllerOrientationAndPositionForTime(ControllerIndex, OpenXRSourceNames::LeftGrip, Time, OutTimeWasUsed, OutOrientation, OutPosition, OutbProvidedLinearVelocity, OutLinearVelocity, OutbProvidedAngularVelocity, OutAngularVelocityAsAxisAndLength, OutbProvidedLinearAcceleration, OutLinearAcceleration, WorldToMetersScale)
				|| GetControllerOrientationAndPositionForTime(ControllerIndex, OpenXRSourceNames::RightGrip, Time, OutTimeWasUsed, OutOrientation, OutPosition, OutbProvidedLinearVelocity, OutLinearVelocity, OutbProvidedAngularVelocity, OutAngularVelocityAsAxisAndLength, OutbProvidedLinearAcceleration, OutLinearAcceleration, WorldToMetersScale);
		}

		if (MotionSource == OpenXRSourceNames::Left)
		{
			return GetControllerOrientationAndPositionForTime(ControllerIndex, OpenXRSourceNames::LeftGrip, Time, OutTimeWasUsed, OutOrientation, OutPosition, OutbProvidedLinearVelocity, OutLinearVelocity, OutbProvidedAngularVelocity, OutAngularVelocityAsAxisAndLength, OutbProvidedLinearAcceleration, OutLinearAcceleration, WorldToMetersScale);
		}

		if (MotionSource == OpenXRSourceNames::Right)
		{
			return GetControllerOrientationAndPositionForTime(ControllerIndex, OpenXRSourceNames::RightGrip, Time, OutTimeWasUsed, OutOrientation, OutPosition, OutbProvidedLinearVelocity, OutLinearVelocity, OutbProvidedAngularVelocity, OutAngularVelocityAsAxisAndLength, OutbProvidedLinearAcceleration, OutLinearAcceleration, WorldToMetersScale);
		}
	}

	XrSession Session = OpenXRHMD->GetSession();

	if (Session == XR_NULL_HANDLE)
	{
		return false;
	}

	XrActionStateGetInfo GetInfo;
	GetInfo.type = XR_TYPE_ACTION_STATE_GET_INFO;
	GetInfo.next = nullptr;
	GetInfo.subactionPath = XR_NULL_PATH;
	GetInfo.action = GetActionForMotionSource(MotionSource);

	if (GetInfo.action == XR_NULL_HANDLE)
	{
		return false;
	}

	XrActionStatePose State;
	State.type = XR_TYPE_ACTION_STATE_POSE;
	State.next = nullptr;
	XrResult Result = xrGetActionStatePose(Session, &GetInfo, &State);
	bool bPoseFetched = false;
	if (Result >= XR_SUCCESS && State.isActive)
	{
		FQuat Orientation;
		bPoseFetched = OpenXRHMD->GetPoseForTime(GetDeviceIDForMotionSource(MotionSource), Time, OutTimeWasUsed, Orientation, OutPosition, OutbProvidedLinearVelocity, OutLinearVelocity, OutbProvidedAngularVelocity, OutAngularVelocityAsAxisAndLength, OutbProvidedLinearAcceleration, OutLinearAcceleration, WorldToMetersScale);
		OutOrientation = FRotator(Orientation);
	}

	return bPoseFetched;
}

ETrackingStatus FOpenXRInputPlugin::FOpenXRInput::GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const
{
	if (!bActionsAttached || OpenXRHMD == nullptr)
	{
		return ETrackingStatus::NotTracked;
	}

	if (ControllerIndex == 0 && IsOpenXRInputSupportedMotionSource(MotionSource))
	{
		if (MotionSource == OpenXRSourceNames::AnyHand)
		{
			if (GetControllerTrackingStatus(ControllerIndex, OpenXRSourceNames::LeftGrip) == ETrackingStatus::Tracked)
			{
				return ETrackingStatus::Tracked;
			}
			else
			{
				return GetControllerTrackingStatus(ControllerIndex, OpenXRSourceNames::RightGrip);
			}
		}

		if (MotionSource == OpenXRSourceNames::Left)
		{
			return GetControllerTrackingStatus(ControllerIndex, OpenXRSourceNames::LeftGrip);
		}

		if (MotionSource == OpenXRSourceNames::Right)
		{
			return GetControllerTrackingStatus(ControllerIndex, OpenXRSourceNames::RightGrip);
		}

		XrSession Session = OpenXRHMD->GetSession();

		if (Session == XR_NULL_HANDLE)
		{
			return ETrackingStatus::NotTracked;
		}

		XrActionStateGetInfo GetInfo;
		GetInfo.type = XR_TYPE_ACTION_STATE_GET_INFO;
		GetInfo.next = nullptr;
		GetInfo.subactionPath = XR_NULL_PATH;
		GetInfo.action = GetActionForMotionSource(MotionSource);

		XrActionStatePose State;
		State.type = XR_TYPE_ACTION_STATE_POSE;
		State.next = nullptr;
		XrResult Result = xrGetActionStatePose(Session, &GetInfo, &State);
		if (XR_SUCCEEDED(Result) && State.isActive)
		{
			bool bIsTracked = OpenXRHMD->GetIsTracked(GetDeviceIDForMotionSource(MotionSource));
			return bIsTracked ? ETrackingStatus::Tracked : ETrackingStatus::NotTracked;
		}
	}

	return ETrackingStatus::NotTracked;
}

void FOpenXRInputPlugin::FOpenXRInput::EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const
{
	check(IsInGameThread());

	SourcesOut.Add(OpenXRSourceNames::Head);
	SourcesOut.Add(OpenXRSourceNames::AnyHand);
	SourcesOut.Add(OpenXRSourceNames::Left);
	SourcesOut.Add(OpenXRSourceNames::Right);
	SourcesOut.Add(OpenXRSourceNames::LeftGrip);
	SourcesOut.Add(OpenXRSourceNames::RightGrip);
	SourcesOut.Add(OpenXRSourceNames::LeftAim);
	SourcesOut.Add(OpenXRSourceNames::RightAim);
	SourcesOut.Add(OpenXRSourceNames::LeftPalm);
	SourcesOut.Add(OpenXRSourceNames::RightPalm);
}

// TODO: Refactor API to change the Hand type to EControllerHand
void FOpenXRInputPlugin::FOpenXRInput::SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values)
{
	if (!bActionsAttached || OpenXRHMD == nullptr)
	{
		return;
	}

	XrSession Session = OpenXRHMD->GetSession();

	if (Session == XR_NULL_HANDLE)
	{
		return;
	}
	
	if (!OpenXRHMD->IsFocused())
	{
		return;
	}

	XrHapticVibration HapticValue;
	HapticValue.type = XR_TYPE_HAPTIC_VIBRATION;
	HapticValue.next = nullptr;
	HapticValue.duration = XrDuration(CurrentDeltaTime * 1e9);
	HapticValue.frequency = Values.Frequency;
	HapticValue.amplitude = Values.Amplitude;

	if (ControllerId == 0)
	{
		FOpenXRController* Controller = nullptr;
		if ((Hand == (int32)EControllerHand::Left || Hand == (int32)EControllerHand::AnyHand) && Controllers.Contains(EControllerHand::Left))
		{
			Controller = Controllers.Find(EControllerHand::Left);
		}
		if ((Hand == (int32)EControllerHand::Right || Hand == (int32)EControllerHand::AnyHand) && Controllers.Contains(EControllerHand::Right))
		{
			Controller = Controllers.Find(EControllerHand::Right);
		}
		if (Hand == (int32)EControllerHand::HMD)
		{
			Controller = Controllers.Find(EControllerHand::HMD);
		}

		if (Controller)
		{
			XrHapticActionInfo HapticActionInfo;
			HapticActionInfo.type = XR_TYPE_HAPTIC_ACTION_INFO;
			HapticActionInfo.next = nullptr;
			HapticActionInfo.subactionPath = XR_NULL_PATH;
			HapticActionInfo.action = Controller->VibrationAction;

			if (Values.HapticBuffer == nullptr && (Values.Amplitude <= 0.0f || Values.Frequency < XR_FREQUENCY_UNSPECIFIED))
			{
				if (Controller->bHapticActive)
				{
					XR_ENSURE(xrStopHapticFeedback(Session, &HapticActionInfo));
				}
				Controller->bHapticActive = false;
			}
			else
			{
				FOpenXRExtensionChainStructPtrs ScopedExtensionChainStructs;
				if (Values.HapticBuffer != nullptr)
				{
					OpenXRHMD->GetIOpenXRExtensionPluginDelegates().GetApplyHapticFeedbackAddChainStructsDelegate().Broadcast(&HapticValue, ScopedExtensionChainStructs, Values.HapticBuffer);
				}
				XR_ENSURE(xrApplyHapticFeedback(Session, &HapticActionInfo, (const XrHapticBaseHeader*)&HapticValue));

				Controller->bHapticActive = true;
			}
		}
	}
}

void FOpenXRInputPlugin::FOpenXRInput::GetHapticFrequencyRange(float& MinFrequency, float& MaxFrequency) const
{
	MinFrequency = XR_FREQUENCY_UNSPECIFIED;
	MaxFrequency = XR_FREQUENCY_UNSPECIFIED;
}

float FOpenXRInputPlugin::FOpenXRInput::GetHapticAmplitudeScale() const
{
	return 1.0f;
}

bool FOpenXRInputPlugin::FOpenXRInput::SetEnhancedInputUserSettings(TObjectPtr<class UEnhancedInputUserSettings> InputSettings)
{
	if (bActionsAttached)
	{
		UE_LOGF(LogHMD, Error, "Attempted to attach a set of enhanced user input settings when actions are already attached for the current session.");

		return false;
	}

	const TSet<TObjectPtr<const UInputMappingContext>>& MappingContexts = InputSettings->GetRegisteredInputMappingContexts();

	for (const auto& Context : MappingContexts)
	{
		InputMappingContextToPriorityMap.Add(TStrongObjectPtr<const UInputMappingContext>(Context), 0);
	}
	return true;
}

bool FOpenXRInputPlugin::FOpenXRInput::AttachInputMappingContexts(const TSet<TObjectPtr<UInputMappingContext>>& MappingContexts)
{
	if (bActionsAttached)
	{
		UE_LOGF(LogHMD, Error, "Attempted to attach input mapping contexts when action sets are already attached for the current session.");

		return false;
	}

	for (const auto& Context : MappingContexts)
	{
		InputMappingContextToPriorityMap.Add(TStrongObjectPtr<UInputMappingContext>(Context), 0);
	}
	return true;
}

#undef LOCTEXT_NAMESPACE // "OpenXRInputPlugin"
