// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "OpenXRCore.h"
#include "GenericPlatform/IInputInterface.h"
#include "XRMotionControllerBase.h"
#include "IOpenXRInputPlugin.h"
#include "IOpenXRExtensionPlugin.h"
#include "IInputDevice.h"
#include "IHapticDevice.h"
#include "UObject/ObjectPtr.h"
#include "UObject/StrongObjectPtr.h"

class IOpenXRHMD;
class IXRTrackingSystem;
class UInputAction;
class UInputTrigger;
class UInputModifier;
class UInputMappingContext;
struct FInputActionKeyMapping;
struct FInputAxisKeyMapping;
struct FKey;

class FOpenXRInputPlugin : public IOpenXRInputPlugin
{
public:
	struct FOpenXRAction
	{
		XrActionSet		Set;
		XrActionType	Type;
		FName			Name;
		XrAction		Handle;

		// Legacy Input
		XrTime			NextRepeatTime;

		// Enhanced Input
		TObjectPtr<const UInputAction> Object;
		TMultiMap<TPair<XrPath, XrPath>, TObjectPtr<UInputTrigger>> Triggers;
		TMultiMap<TPair<XrPath, XrPath>, TObjectPtr<UInputModifier>> Modifiers;

		FOpenXRAction(XrActionSet InActionSet,
			XrActionType InActionType,
			const FName& InName, const
			FString& InLocalizedName,
			const TArray<XrPath>& InSubactionPaths,
			const TObjectPtr<const UInputAction>& InObject,
			IOpenXRHMD* OpenXRHMD = nullptr);

		FOpenXRAction(XrActionSet InActionSet,
			XrActionType InActionType,
			const FName& InName,
			const FString& InLocalizedName,
			const TArray<XrPath>& InSubactionPaths,
			IOpenXRHMD* OpenXRHMD = nullptr);
	};

	struct FOpenXRActionSet
	{
		XrActionSet		Handle;
		FName			Name;
		FString			LocalizedName;

		TObjectPtr<const UInputMappingContext> Object;

		FOpenXRActionSet(XrInstance InInstance,
			const FName& InName,
			const FString& InLocalizedName,
			uint32 InPriority,
			const TObjectPtr<const UInputMappingContext>& InObject,
			IOpenXRHMD* OpenXRHMD = nullptr);

		FOpenXRActionSet(XrInstance InInstance,
			const FName& InName,
			const FString& InLocalizedName,
			uint32 InPriority,
			IOpenXRHMD* OpenXRHMD = nullptr);
	};

	struct FOpenXRController
	{
		XrActionSet		ActionSet;
		XrPath			UserPath;
		XrAction		GripAction;
		XrAction		AimAction;
		XrAction		PalmAction;
		XrAction		VibrationAction;
		int32			GripDeviceId;
		int32			AimDeviceId;
		int32			PalmDeviceId;

		bool			bHapticActive;

		FOpenXRController(XrActionSet InActionSet, XrPath InUserPath, const char* InName);

		void AddTrackedDevices(IOpenXRHMD* HMD);
	};

	struct FInteractionProfile
	{
	public:
		bool HasHaptics;
		XrPath Path;
		TArray<XrActionSuggestedBinding> Bindings;

		FInteractionProfile(XrPath InProfile, bool InHasHaptics);
	};

	class FOpenXRInput : public IOpenXRInputModule, public IInputDevice, public FXRMotionControllerBase, public IHapticDevice, public TSharedFromThis<FOpenXRInput>
	{
	public:
		FOpenXRInput(IXRTrackingSystem* InTrackingSystem);
		virtual ~FOpenXRInput() {};

		// IOpenXRAdditionalModule overrides
		virtual void OnBeginSession() override;
		virtual void OnDestroySession() override;

		// IInputDevice overrides
		virtual void Tick(float DeltaTime) override { CurrentDeltaTime = DeltaTime; };
		virtual void SendControllerEvents() override;
		virtual void SetMessageHandler(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler) override;
		virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
		virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
		virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues &values) override;
		virtual bool SupportsForceFeedback(int32 ControllerId) override;
		virtual void SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property) override;
		// IMotionController overrides
		virtual FName GetMotionControllerDeviceTypeName() const override;
		virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const override;
		virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const override;
		virtual bool GetControllerOrientationAndPositionForTime(const int32 ControllerIndex, const FName MotionSource, FTimespan Time, bool& OutTimeWasUsed, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const override;
		virtual ETrackingStatus GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const override;
		virtual void EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const override;
		virtual bool SetEnhancedInputUserSettings(TObjectPtr<class UEnhancedInputUserSettings> InputSettings) override;
		virtual bool AttachInputMappingContexts(const TSet<TObjectPtr<UInputMappingContext>>& MappingContexts) override;

		// IHapticDevice overrides
		IHapticDevice* GetHapticDevice() override { return (IHapticDevice*)this; }
		virtual void SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values) override;

		virtual void GetHapticFrequencyRange(float& MinFrequency, float& MaxFrequency) const override;
		virtual float GetHapticAmplitudeScale() const override;

	private:
		IXRTrackingSystem* TrackingSystem;
		IOpenXRHMD* OpenXRHMD;
		XrInstance Instance;

		TUniquePtr<FOpenXRActionSet> ControllerActionSet;
		TArray<FOpenXRActionSet> ActionSets;
		TArray<XrPath> SubactionPaths;
		TArray<FOpenXRAction> LegacyActions, EnhancedActions;
		TMap<EControllerHand, FOpenXRController> Controllers;
		TMap<FName, EControllerHand> MotionSourceToControllerHandMap;
		TMap<FString, TArray<FInputKeyOpenXRProperties>> InputsKeysToPropertiesMap;

		// Holds Enhanced Input Mapping Contexts pulled from the Enhanced Input Settings and mapped to their priority
		// These will be converted into OpenXR action sets
		TMap<TStrongObjectPtr<const UInputMappingContext>, uint32> InputMappingContextToPriorityMap;

		XrAction GetActionForMotionSource(FName MotionSource) const;
		int32 GetDeviceIDForMotionSource(FName MotionSource) const;
		XrPath GetUserPathForMotionSource(FName MotionSource) const;
		bool IsOpenXRInputSupportedMotionSource(const FName MotionSource) const;

		bool bActionsAttached;
		bool bDirectionalBindingSupported;
		bool bPalmPoseSupported;
		bool bGripSurfacePoseSupported;
		bool bActionSetPrioritySupported;

		/**
		* Buffer for current delta time to get an accurate approximation of how long to play haptics for
		*/
		float CurrentDeltaTime = 0.0f;

		/** Repeat key delays */
		const XrTime InitialButtonRepeatDelay = 2e8;
		const XrTime ButtonRepeatDelay = 1e8;

		bool BuildActions(XrSession Session);
		void SyncActions(XrSession Session);
		void BuildLegacyActions(TMap<FString, FInteractionProfile>& Profiles);
		void BuildEnhancedActions(TMap<FString, FInteractionProfile>& Profiles);
		void DestroyActions();

		template<typename T>
		int32 SuggestBindings(TMap<FString, FInteractionProfile>& Profiles, FOpenXRAction& Action, const TArray<T>& Mappings);
		bool SuggestBindingForKey(TMap<FString, FInteractionProfile>& Profiles, FOpenXRAction& Action, const FKey& Key, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers);
		bool SuggestBindingForKey(TMap<FString, FInteractionProfile>& Profiles, FOpenXRAction& Action, const FKey& Key);

		/** handler to send all messages to */
		TSharedRef<FGenericApplicationMessageHandler> MessageHandler;
	};

	FOpenXRInputPlugin();
	virtual ~FOpenXRInputPlugin();

	virtual void StartupModule() override;
	virtual TSharedPtr< class IInputDevice > CreateInputDevice(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler) override;

private:
	TSharedPtr<FOpenXRInput> InputDevice;
};
