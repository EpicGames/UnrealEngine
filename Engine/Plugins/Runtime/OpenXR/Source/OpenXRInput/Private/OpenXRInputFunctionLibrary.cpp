// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRInputFunctionLibrary.h"

#include "Engine/Engine.h"
#include "Features/IModularFeatures.h"
#include "HeadMountedDisplayFunctionLibrary.h"
#include "PlayerMappableInputConfig.h"
#include "UObject/ObjectPtr.h"
#include "XRMotionControllerBase.h"

UOpenXRInputFunctionLibrary::UOpenXRInputFunctionLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UOpenXRInputFunctionLibrary::BeginXRSession(const TSet<UInputMappingContext*>& InputMappingContexts)
{
	TSet<TObjectPtr<UInputMappingContext>> Contexts;
	for (UInputMappingContext* Context : InputMappingContexts)
	{
		Contexts.Add(Context);
	}

	TArray<IMotionController*> MotionControllers = IModularFeatures::Get().GetModularFeatureImplementations<IMotionController>(IMotionController::GetModularFeatureName());
	for (auto MotionController : MotionControllers)
	{
		if (MotionController == nullptr)
		{
			continue;
		}

		MotionController->AttachInputMappingContexts(Contexts);
	}

	return UHeadMountedDisplayFunctionLibrary::EnableHMD(true);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UOpenXRInputFunctionLibrary::EndXRSession()
{
	UHeadMountedDisplayFunctionLibrary::EnableHMD(false);
}
