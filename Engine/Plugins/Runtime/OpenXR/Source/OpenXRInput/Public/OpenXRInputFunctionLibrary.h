// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "OpenXRInputFunctionLibrary.generated.h"

UCLASS()
class OPENXRINPUT_API UOpenXRInputFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	* Begin an XR session with the provided input mapping. This will make the tracking system aware of the actions and bindings that are used for XR motion controllers.
	* Attaching input configs to the session can only be done once, so this is a helper function to attach the input mapping contexts and start the XR session correctly.
	* 
	* @param InputMappingContexts		The set of input mapping contexts used for XR
	*
	* @return			False if the input mapping contexts can't be attached to the session, true otherwise
	*/
	UFUNCTION(BlueprintCallable, Category = "Input|XRTracking")
	static bool BeginXRSession(const TSet<UInputMappingContext*>& InputMappingContexts);


	UFUNCTION(BlueprintCallable, Category = "Input|XRTracking")
	static void EndXRSession();
};
